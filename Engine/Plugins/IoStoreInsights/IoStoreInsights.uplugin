{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "IoStore Insights",
	"Description": "Allows capturing IoStore activity via Unreal Insights",
	"Category": "Insights",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Modules": [
		{
			"Name": "IoStoreInsights",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default",
			"ProgramAllowList": [
				"UnrealInsights"
			]
		}
	]
}
