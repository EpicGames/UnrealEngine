// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class IoStoreInsights : ModuleRules
	{
		public IoStoreInsights(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"SlateCore",
				"Slate",
				"InputCore",
				"TraceAnalysis",
				"TraceServices",
				"TraceInsights",
				"TraceInsightsCore",
			});

		}
	}
}

