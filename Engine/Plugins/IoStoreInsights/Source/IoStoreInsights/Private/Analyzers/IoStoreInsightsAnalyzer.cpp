// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analyzers/IoStoreInsightsAnalyzer.h"
#include "Model/IoStoreInsightsProvider.h"
#include "TraceServices/Model/MetadataProvider.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/Strings.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/ProviderLock.h"
#include "HAL/LowLevelMemTracker.h"

namespace UE::IoStoreInsights
{
	FIoStoreInsightsAnalyzer::FIoStoreInsightsAnalyzer(TraceServices::IAnalysisSession& InSession, FIoStoreInsightsProvider& InIoStoreProvider)
		: Session(InSession)
		, Provider(InIoStoreProvider)
	{
	}



	void FIoStoreInsightsAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
	{
		Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

		Builder.RouteEvent(RouteId_BackendName,      "IoStore",  "BackendName");
		Builder.RouteEvent(RouteId_RequestCreate,    "IoStore",  "RequestCreate");
		Builder.RouteEvent(RouteId_RequestStarted,   "IoStore",  "RequestStarted");
		Builder.RouteEvent(RouteId_RequestCompleted, "IoStore",  "RequestCompleted");
		Builder.RouteEvent(RouteId_RequestFailed,    "IoStore",  "RequestFailed");
		Builder.RouteEvent(RouteId_PackageMapping,   "Package",  "PackageMapping");
	}



	bool FIoStoreInsightsAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
	{
		LLM_SCOPE_BYNAME(TEXT("Insights/FIoStoreInsightsAnalyzer"));
		const auto& EventData = Context.EventData;

		TraceServices::FAnalysisSessionEditScope _(Session);
		switch (RouteId)
		{
			case RouteId_BackendName:
			{
				const uint64 Backend = EventData.GetValue<uint64>("BackendHandle");
				FString BackendName;
				if (EventData.GetString("Name", BackendName))
				{
					Provider.AddBackendName(Backend, Session.StoreString(BackendName));
				}
			}
			break;

			case RouteId_RequestCreate:
			{
				const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
				double Time = Context.EventTime.AsSeconds(Cycle);
				Session.UpdateDurationSeconds(Time);

				const uint64 RequestHandle = EventData.GetValue<uint64>("RequestHandle");
				const uint64 BatchHandle   = EventData.GetValue<uint64>("BatchHandle");
				const uint32 CallstackId   = EventData.GetValue<uint32>("CallstackId");
				const uint64 Offset        = EventData.GetValue<uint64>("Offset");
				const uint64 Size          = EventData.GetValue<uint64>("Size");
				const uint32 ChunkIdHash   = EventData.GetValue<uint32>("ChunkIdHash");
				const uint8 ChunkType      = EventData.GetValue<uint8>("ChunkType");
				const uint32 ThreadId      = Context.ThreadInfo.GetId();
				if (!ActiveRequestsMap.Contains(RequestHandle))
				{
					FString PackageName, ExtraTag;
					uint64 PackageId;
					GetPackageDetailFromMetadata(Context, ThreadId, PackageName, ExtraTag, PackageId);
					uint32 IoStoreRequestIndex = Provider.GetIoStoreRequestIndex(ChunkIdHash, ChunkType, Offset, Size, CallstackId, PackageId, *PackageName, *ExtraTag);

					uint64 ActivityIndex = Provider.BeginIoStoreActivity(IoStoreRequestIndex, EIoStoreActivityType::Request_Pending, ThreadId, 0, Time);
					ActiveRequestsMap.Emplace( RequestHandle, {IoStoreRequestIndex, ActivityIndex} );
				}
				else
				{
					//IOSTORETRACE_WARNING("RequestCreate: Request already started!");
				}
			}
			break;

			case RouteId_RequestUnresolved:
			{
				const uint64 Cycle			= EventData.GetValue<uint64>("Cycle");
				const double Time			= Context.EventTime.AsSeconds(Cycle);

				Session.UpdateDurationSeconds(Time);

				const uint64 RequestHandle	= EventData.GetValue<uint64>("RequestHandle");
				const uint32 ThreadId		= Context.ThreadInfo.GetId();
				const uint64 Size			= 0;
				const bool bFailed			= true;

				if (const FPendingActivity* FindRead = ActiveReadsMap.Find(RequestHandle))
				{
					Provider.EndIoStoreActivity(FindRead->IoStoreRequestIndex, FindRead->ActivityIndex, Size, bFailed, Time);
					ActiveReadsMap.Remove(RequestHandle);
				}
				ActiveRequestsMap.Remove(RequestHandle); // note: we don't have a separate "request destroy", so clean everything up once it has been read
			}
			break;

			case RouteId_RequestStarted:
			{
				const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
				double Time = Context.EventTime.AsSeconds(Cycle);
				Session.UpdateDurationSeconds(Time);

				const uint64 RequestHandle = EventData.GetValue<uint64>("RequestHandle");
				const uint64 BackendHandle = EventData.GetValue<uint64>("BackendHandle");
				const uint32 ThreadId      = Context.ThreadInfo.GetId();
			
				const FPendingRequest* Request = ActiveRequestsMap.Find(RequestHandle);
				if (Request == nullptr)
				{
					//IOSTORETRACE_WARNING("RequestStarted: request was not created!");
					Request = &ActiveRequestsMap.Emplace( RequestHandle, {Provider.GetUnknownIoStoreRequestIndex(), 0} );
				}
				else
				{
					Provider.EndIoStoreActivity(Request->IoStoreRequestIndex, Request->CreateActivityIndex, 0, false, Time);
				}

				uint64 ReadIndex = Provider.BeginIoStoreActivity(Request->IoStoreRequestIndex, EIoStoreActivityType::Request_Read, ThreadId, BackendHandle, Time);
				FPendingActivity& Read = ActiveReadsMap.Add(RequestHandle);
				Read.IoStoreRequestIndex = Request->IoStoreRequestIndex;
				Read.ActivityIndex = ReadIndex;
			}
			break;

			case RouteId_RequestCompleted:
			case RouteId_RequestFailed:
			{
				const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
				double Time = Context.EventTime.AsSeconds(Cycle);
				Session.UpdateDurationSeconds(Time);

				const bool bFailed         = (RouteId == RouteId_RequestFailed);
				const uint64 RequestHandle = EventData.GetValue<uint64>("RequestHandle");
				const uint64 Size          = bFailed ? 0 : EventData.GetValue<uint64>("Size");

				const FPendingActivity* FindRead = ActiveReadsMap.Find(RequestHandle);
				if (FindRead)
				{
					Provider.EndIoStoreActivity(FindRead->IoStoreRequestIndex, FindRead->ActivityIndex, Size, bFailed, Time);
					ActiveReadsMap.Remove(RequestHandle);
					ActiveRequestsMap.Remove(RequestHandle); // note: we don't have a separate "request destroy", so clean everything up once it has been read
				}
				else
				{
					//IOSTORETRACE_WARNING("RequestCompleted: RequestStarted not traced!");
				}
			}
			break;

			case RouteId_PackageMapping:
			{
				const TraceServices::IDefinitionProvider* DefinitionProvider = TraceServices::ReadDefinitionProvider(Session);
				if (DefinitionProvider)
				{
					const uint64 PackageId    = EventData.GetValue<uint64>("Id");
					const auto PackageNameRef = EventData.GetReferenceValue<uint32>("Package");
					const auto PackageName    = DefinitionProvider->Get<TraceServices::FStringDefinition>(PackageNameRef);
					if (PackageName && *PackageName->Display != '\0')
					{
						Provider.AddPackageMapping(PackageId, PackageName->Display);
					}
				}
			}
			break;

		}
		return true;
	}



	bool FIoStoreInsightsAnalyzer::GetPackageDetailFromMetadata( const FOnEventContext& Context, uint32 ThreadId, FString& OutPackageName, FString& OutExtraTag, uint64& OutPackageId ) const
	{
		const TraceServices::IMetadataProvider* MetadataProvider = ReadMetadataProvider(Session);

		// lookup metadata id
		uint32 MetadataId = TraceServices::IMetadataProvider::InvalidMetadataId;
		{
			TraceServices::IEditableMetadataProvider* EditableMetadataProvider = EditMetadataProvider(Session);
			TraceServices::FProviderEditScopeLock MetadataProviderEditLock(*EditableMetadataProvider);

			MetadataId = EditableMetadataProvider->PinAndGetId(ThreadId);
		}
		if (MetadataId == TraceServices::IMetadataProvider::InvalidMetadataId)
		{
			return false;
		}

		// lookup metadata types (must be nicer way to do this!)
		struct FMetadataType
		{
			uint16 MetadataType = 0;
			const TraceServices::FMetadataSchema* Schema = nullptr;

			void Init(const TraceServices::IMetadataProvider* MetadataProvider, int& PendingMetadata, FName Name)
			{
				MetadataType = MetadataProvider->GetRegisteredMetadataType(Name);
				if (MetadataType != TraceServices::IMetadataProvider::InvalidMetadataId)
				{
					Schema = MetadataProvider->GetRegisteredMetadataSchema(MetadataType);
					if (Schema)
					{
						PendingMetadata++;
					}
				}
			}
		} MetadataTypes[3];
		int PendingMetadata = 0;
		{
			TraceServices::FProviderReadScopeLock MetadataProviderReadLock(*MetadataProvider);
			MetadataTypes[0].Init(MetadataProvider, PendingMetadata, TEXT("Asset"));
			MetadataTypes[1].Init(MetadataProvider, PendingMetadata, TEXT("PackageId"));
			MetadataTypes[2].Init(MetadataProvider, PendingMetadata, TEXT("IoStoreTag"));
		}

		// find the asset metadata associated with the current thread callstack
		const TraceServices::IDefinitionProvider* DefinitionProvider = TraceServices::ReadDefinitionProvider(Session);
		if (MetadataId != TraceServices::IMetadataProvider::InvalidMetadataId && DefinitionProvider && PendingMetadata > 0)
		{
			TraceServices::FProviderReadScopeLock MetadataProviderReadLock(*MetadataProvider);
			MetadataProvider->EnumerateMetadata(ThreadId, MetadataId,
				[&MetadataTypes, &PendingMetadata, DefinitionProvider, &OutPackageName, &OutPackageId, &OutExtraTag](uint32 StackDepth, uint16 Type, const void* Data, uint32 Size) -> bool
				{
					if (Type == MetadataTypes[0].MetadataType && MetadataTypes[0].Schema) // Asset metadata
					{
						TraceServices::FProviderReadScopeLock DefinitionProviderReadLock(*DefinitionProvider);
						const auto Reader = MetadataTypes[0].Schema->Reader();
						const auto PackageNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 2); // { 0:Name, 1:Class, 2:Package }
						const auto PackageName = DefinitionProvider->Get<TraceServices::FStringDefinition>(*PackageNameRef);
						if (PackageName)
						{
							OutPackageName = PackageName->Display;
						}
						PendingMetadata--;
					}
					else if (Type == MetadataTypes[1].MetadataType && MetadataTypes[1].Schema) // PackageId metadata
					{
						TraceServices::FProviderReadScopeLock DefinitionProviderReadLock(*DefinitionProvider);
						const auto Reader = MetadataTypes[1].Schema->Reader();
						const auto PackageId = Reader.GetValueAs<uint64>((uint8*)Data, 0); // { 0:PackageId }
						if (PackageId)
						{
							OutPackageId = *PackageId;
						}
						PendingMetadata--;
					}
					else if (Type == MetadataTypes[2].MetadataType && MetadataTypes[2].Schema) // IoStoreTag metadata
					{
						TraceServices::FProviderReadScopeLock DefinitionProviderReadLock(*DefinitionProvider);
						const auto Reader = MetadataTypes[2].Schema->Reader();
						const auto TagNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 0); // { 0:Tag }
						const auto TagName = DefinitionProvider->Get<TraceServices::FStringDefinition>(*TagNameRef);
						if (TagName)
						{
							OutExtraTag = TagName->Display;
						}

						PendingMetadata--;
					}
					return (PendingMetadata > 0);
				});

			return true;
		}

		return false;
	}

} //namespace UE::IoStoreInsights

