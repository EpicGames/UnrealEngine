// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Containers/Map.h"

namespace TraceServices
{ 
	class IAnalysisSession; 
}

namespace UE::IoStoreInsights
{ 
	class FIoStoreInsightsProvider;

	class FIoStoreInsightsAnalyzer : public UE::Trace::IAnalyzer
	{
	public:
		FIoStoreInsightsAnalyzer(TraceServices::IAnalysisSession& InSession, FIoStoreInsightsProvider& InIoStoreProvider);

		virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
		virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

	private:
		bool GetPackageDetailFromMetadata( const FOnEventContext& Context, uint32 ThreadId, FString& OutPackageName, FString& OutExtraTag, uint64& OutPackageId ) const;

		enum : uint16
		{
			RouteId_BackendName,
			RouteId_RequestCreate,
			RouteId_RequestStarted,
			RouteId_RequestCompleted,
			RouteId_RequestFailed,
			RouteId_PackageMapping,
			RouteId_RequestUnresolved,
		};

		struct FPendingActivity
		{
			uint64 ActivityIndex;
			uint32 IoStoreRequestIndex;
		};

		struct FPendingRequest
		{
			uint32 IoStoreRequestIndex;
			uint64 CreateActivityIndex;
		};

		TraceServices::IAnalysisSession& Session;
		FIoStoreInsightsProvider& Provider;
		TMap<uint64, FPendingActivity> ActiveReadsMap;
		TMap<uint64, FPendingRequest> ActiveRequestsMap;
		TMap<uint64, uint64> ActiveBatchMap;
	};

} // namespace UE::IoStoreInsights
