// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreInsightsModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"

#include "Insights/IUnrealInsightsModule.h"
#include "Widgets/SIoStoreAnalysisTab.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "IoStoreInsightsModule"

namespace UE::IoStoreInsights
{
	namespace Private
	{
		const FName ViewTab("IoStoreViewTab");
	}



	FIoStoreInsightsModule& FIoStoreInsightsModule::Get()
	{
		return FModuleManager::LoadModuleChecked<FIoStoreInsightsModule>("IoStoreInsights");
	}



	void FIoStoreInsightsModule::StartupModule()
	{
		IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
		IModularFeatures::Get().RegisterModularFeature(Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);

		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		FOnRegisterMajorTabExtensions& TimingProfilerLayoutExtension = UnrealInsightsModule.OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId);
		TimingProfilerLayoutExtension.AddRaw(this, &FIoStoreInsightsModule::RegisterTimingProfilerLayoutExtensions);
	}



	void FIoStoreInsightsModule::ShutdownModule()
	{
		if (IUnrealInsightsModule* UnrealInsightsModule = FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"))
		{
			FOnRegisterMajorTabExtensions& TimingProfilerLayoutExtension = UnrealInsightsModule->OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId);
			TimingProfilerLayoutExtension.RemoveAll(this);
		}

		IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
		IModularFeatures::Get().UnregisterModularFeature(Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
	}



	void FIoStoreInsightsModule::RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender)
	{
		InsightsTabManager = InOutExtender.GetTabManager();

		FMinorTabConfig& MinorTabConfig = InOutExtender.AddMinorTabConfig();
		MinorTabConfig.TabId = Private::ViewTab;
		MinorTabConfig.TabLabel = LOCTEXT("IoStoreTabTitle", "IoStore View");
		MinorTabConfig.TabTooltip = LOCTEXT("IoStoreTabTitleTooltip", "Opens the IoStore View tab, allows for diagnostics of IoStore data.");
		MinorTabConfig.TabIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Plugin.TreeItem");
		MinorTabConfig.WorkspaceGroup = InOutExtender.GetWorkspaceGroup();
		MinorTabConfig.OnSpawnTab = FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
		{
			return SNew(SDockTab)
				.ShouldAutosize(false)
				.TabRole(ETabRole::PanelTab)
				[
					SAssignNew(IoStoreAnalysisView, SIoStoreAnalysisTab)
				];
		});

		InOutExtender.GetLayoutExtender().ExtendLayout(Private::ViewTab, ELayoutExtensionPosition::Before, FTabManager::FTab(Private::ViewTab, ETabState::ClosedTab));
	}



	TSharedPtr<SIoStoreAnalysisTab> FIoStoreInsightsModule::GetIoStoreAnalysisViewTab(bool bInvoke)
	{
		if (bInvoke)
		{
			if (TSharedPtr<FTabManager> TabManagerPin = InsightsTabManager.Pin())
			{
				TabManagerPin->TryInvokeTab(Private::ViewTab);
			}
		}
		return IoStoreAnalysisView.Pin();
	}

} //namespace UE::IoStoreInsights

IMPLEMENT_MODULE(UE::IoStoreInsights::FIoStoreInsightsModule, IoStoreInsights);

#undef LOCTEXT_NAMESPACE
