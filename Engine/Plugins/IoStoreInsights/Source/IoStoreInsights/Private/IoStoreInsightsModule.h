// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "IoStoreInsightsTraceModule.h"
#include "ViewModels/IoStoreInsightsTimingViewExtender.h"

struct FInsightsMajorTabExtender;
class FTabManager;

namespace UE::IoStoreInsights
{
	class SIoStoreAnalysisTab;

	class FIoStoreInsightsModule : public IModuleInterface
	{
	public:
		static FIoStoreInsightsModule& Get();

		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

		TSharedPtr<SIoStoreAnalysisTab> GetIoStoreAnalysisViewTab(bool bInvoke);

	private:
		void RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender);
		TWeakPtr<SIoStoreAnalysisTab> IoStoreAnalysisView;
		TWeakPtr<FTabManager> InsightsTabManager;

		FIoStoreInsightsTraceModule TraceModule;
		FIoStoreInsightsTimingViewExtender TimingViewExtender;
	};

} // namespace UE::IoStoreInsights
