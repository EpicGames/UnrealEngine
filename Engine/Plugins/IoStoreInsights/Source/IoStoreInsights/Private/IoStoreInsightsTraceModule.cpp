// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreInsightsTraceModule.h"
#include "Model/IoStoreInsightsProvider.h"
#include "Analyzers/IoStoreInsightsAnalyzer.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace UE::IoStoreInsights
{
	FName FIoStoreInsightsTraceModule::ModuleName("TraceModule_IoStore");



	void FIoStoreInsightsTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
	{
		OutModuleInfo.Name = ModuleName;
		OutModuleInfo.DisplayName = TEXT("IoStore");
	}



	void FIoStoreInsightsTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
	{
		TSharedPtr<FIoStoreInsightsProvider> IoStoreProvider = MakeShared<FIoStoreInsightsProvider>(InSession);
		InSession.AddProvider(FIoStoreInsightsProvider::ProviderName, IoStoreProvider);
		InSession.AddAnalyzer(new FIoStoreInsightsAnalyzer(InSession, *IoStoreProvider));
	}



	void FIoStoreInsightsTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
	{
		OutLoggers.Add(TEXT("IoStore"));
	}



	void FIoStoreInsightsTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
	{
	}

} //namespace UE::IoStoreInsights

