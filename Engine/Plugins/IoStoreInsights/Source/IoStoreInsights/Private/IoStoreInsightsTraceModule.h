// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/ModuleService.h"
#include "ViewModels/IoStoreInsightsTimingViewExtender.h"

namespace UE::IoStoreInsights
{
	class FIoStoreInsightsTraceModule : public TraceServices::IModule
	{
	public:
		virtual ~FIoStoreInsightsTraceModule() = default;

		// TraceServices::IModule interface
		virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
		virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
		virtual void GetLoggers(TArray<const TCHAR *>& OutLoggers) override;
		virtual void GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override;
		virtual const TCHAR* GetCommandLineArgument() override { return TEXT("iostoretrace"); }

	private:
		static FName ModuleName;
	};

} // namespace UE::IoStoreInsights
