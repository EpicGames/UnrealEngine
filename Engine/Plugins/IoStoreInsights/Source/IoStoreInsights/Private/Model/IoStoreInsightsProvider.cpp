// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreInsightsProvider.h"

namespace UE::IoStoreInsights
{
	FName IIoStoreInsightsProvider::ProviderName("IoStoreProvider");



	FIoStoreInsightsProvider::FIoStoreInsightsProvider(TraceServices::IAnalysisSession& InSession)
		: Session(InSession)
		, IoStoreRequests(InSession.GetLinearAllocator(), 1024)
		, IoStoreRequestStates(InSession.GetLinearAllocator(), 1024)
	{
	}



	void FIoStoreInsightsProvider::EnumerateIoStoreRequests(TFunctionRef<bool(const FIoStoreRequest&, const Timeline&)> Callback) const
	{
		for (uint64 IoStoreRequestIndex = 0, IoStoreRequestCount = IoStoreRequests.Num(); IoStoreRequestIndex < IoStoreRequestCount; ++IoStoreRequestIndex)
		{
			const FIoStoreRequestInfoInternal& IoStoreRequestInfoInternal = IoStoreRequests[IoStoreRequestIndex];
			if (!Callback(IoStoreRequestInfoInternal.IoStoreRequestInfo, *IoStoreRequestInfoInternal.ActivityTimeline))
			{
				return;
			}
		}
	}



	uint32 FIoStoreInsightsProvider::GetIoStoreRequestIndex(uint32 ChunkIdHash, uint8 ChunkType, uint64 Offset, uint64 Size, uint32 CallstackId, uint64 PackageId, const TCHAR* PackageName, const TCHAR* ExtraTag)
	{
		uint32 IoStoreRequestIndex = static_cast<uint32>(IoStoreRequests.Num());
		FIoStoreRequestInfoInternal& IoStoreRequestInfo = IoStoreRequests.PushBack();
		IoStoreRequestInfo.IoStoreRequestInfo.IoStoreRequestIndex = IoStoreRequestIndex;
		IoStoreRequestInfo.IoStoreRequestInfo.ChunkIdHash = ChunkIdHash;
		IoStoreRequestInfo.IoStoreRequestInfo.ChunkType = ChunkType;
		IoStoreRequestInfo.IoStoreRequestInfo.Offset = Offset;
		IoStoreRequestInfo.IoStoreRequestInfo.Size = Size;
		IoStoreRequestInfo.IoStoreRequestInfo.PackageId = PackageId;
		IoStoreRequestInfo.IoStoreRequestInfo.CallstackId = CallstackId;
		IoStoreRequestInfo.IoStoreRequestInfo.PackageName = Session.StoreString(PackageName);
		IoStoreRequestInfo.IoStoreRequestInfo.ExtraTag = Session.StoreString(ExtraTag);
		IoStoreRequestInfo.ActivityTimeline = MakeShared<TimelineInternal>(Session.GetLinearAllocator());

		if (*PackageName == '\0' && PackageId != 0)
		{
			TArray<uint32>& PendingPackageRequestIndices = PendingPackageNameMap.FindOrAdd(PackageId);
			PendingPackageRequestIndices.Add(IoStoreRequestIndex);
		}
			

		return IoStoreRequestIndex;
	}



	uint32 FIoStoreInsightsProvider::GetUnknownIoStoreRequestIndex()
	{
		return GetIoStoreRequestIndex(0, 0, 0, 0, 0, 0, TEXT("unknown"),TEXT(""));
	}



	uint64 FIoStoreInsightsProvider::BeginIoStoreActivity(uint32 IoStoreRequestIndex, EIoStoreActivityType Type, uint32 ThreadId, uint64 BackendHandle, double Time)
	{
		FIoStoreRequestInfoInternal& IoStoreRequestInfo = IoStoreRequests[IoStoreRequestIndex];
		FIoStoreActivity& IoStoreActivity = IoStoreRequestStates.PushBack();
		IoStoreActivity.IoStoreRequest = &IoStoreRequestInfo.IoStoreRequestInfo;
		IoStoreActivity.ActualSize = 0;
		IoStoreActivity.StartTime = Time;
		IoStoreActivity.EndTime = std::numeric_limits<double>::infinity();
		IoStoreActivity.ThreadId = ThreadId;
		IoStoreActivity.ActivityType = Type;
		IoStoreActivity.Failed = false;

		const TCHAR** BackendNamePtr = BackendNameMap.Find(BackendHandle);
		if (BackendNamePtr)
		{
			IoStoreActivity.BackendName = (*BackendNamePtr);
		}
		else
		{
			IoStoreActivity.BackendName = TEXT("(Unknown)");
			TArray<FIoStoreActivity*> PendingBackendNameItems = PendingBackendNameMap.FindOrAdd(BackendHandle);
			PendingBackendNameItems.Add(&IoStoreActivity);
		}
		
	
		return IoStoreRequestInfo.ActivityTimeline->AppendBeginEvent(Time, &IoStoreActivity);
	}



	void FIoStoreInsightsProvider::EndIoStoreActivity(uint32 IoStoreRequestIndex, uint64 ActivityIndex, uint64 ActualSize, bool Failed, double Time)
	{
		FIoStoreRequestInfoInternal& IoStoreRequestInfo = IoStoreRequests[IoStoreRequestIndex];
		FIoStoreActivity* Activity = IoStoreRequestInfo.ActivityTimeline->EndEvent(ActivityIndex, Time);
		check(Activity->IoStoreRequest == &IoStoreRequestInfo.IoStoreRequestInfo);
		Activity->ActualSize = ActualSize;
		Activity->Failed = Failed;
		Activity->EndTime = Time;
	}



	void FIoStoreInsightsProvider::AddPackageMapping(uint64 PackageId, const TCHAR* PackageName)
	{
		PackageMap.Add(PackageId, PackageName);

		// see if there are any requests that referred to this package & update the package name now
		const TArray<uint32>* PendingPackageRequestIndicesPtr = PendingPackageNameMap.Find(PackageId);
		if (PendingPackageRequestIndicesPtr != nullptr && *PackageName != '\0')
		{
			const TCHAR* StoredPackageName = Session.StoreString(PackageName);
			for (uint32 IoStoreRequestIndex : *PendingPackageRequestIndicesPtr)
			{
				FIoStoreRequestInfoInternal& IoStoreRequestInfoInternal = IoStoreRequests[IoStoreRequestIndex];
				IoStoreRequestInfoInternal.IoStoreRequestInfo.PackageName = StoredPackageName;
			}

			PendingPackageNameMap.Remove(PackageId);
		}
	}



	void FIoStoreInsightsProvider::AddBackendName(uint64 BackendHandle, const TCHAR* BackendName)
	{
		BackendNameMap.Add(BackendHandle, BackendName);

		const TArray<FIoStoreActivity*>* PendingBackendNameItemsPtr = PendingBackendNameMap.Find(BackendHandle);
		if (PendingBackendNameItemsPtr != nullptr && *BackendName != '\0')
		{
			for ( FIoStoreActivity* IoStoreActivity : (*PendingBackendNameItemsPtr) )
			{
				IoStoreActivity->BackendName = BackendName;
			}

			PendingBackendNameMap.Remove(BackendHandle);
		}
	}



	const FIoStoreRequest& FIoStoreInsightsProvider::GetIoStoreRequest(uint32 IoStoreRequestIndex) const
	{
		check(IoStoreRequestIndex < IoStoreRequests.Num());
		return IoStoreRequests[IoStoreRequestIndex].IoStoreRequestInfo;
	}



	const IIoStoreInsightsProvider* ReadIoStoreInsightsProvider(const TraceServices::IAnalysisSession& Session)
	{
		Session.ReadAccessCheck();
		return Session.ReadProvider<IIoStoreInsightsProvider>(FIoStoreInsightsProvider::ProviderName);
	}



	const TCHAR* LexToString(EIoStoreActivityType ActivityType)
	{
		static const TCHAR* Items[(int)EIoStoreActivityType::Count] =
		{
			TEXT("Pending"),
			TEXT("Read"),
		};

		return (ActivityType < EIoStoreActivityType::Count) ? Items[(int)ActivityType] : TEXT("Invalid");
	}

} // namespace UE::IoStoreInsights