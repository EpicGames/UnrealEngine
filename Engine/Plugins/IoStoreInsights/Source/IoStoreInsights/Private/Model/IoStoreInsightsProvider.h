// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/AnalysisSession.h"
#include "IIoStoreInsightsProvider.h"

namespace TraceServices 
{ 
	class IAnalysisSession; 
}

namespace UE::IoStoreInsights
{
	class FIoStoreInsightsProvider : public IIoStoreInsightsProvider
	{
	public:
		struct FTimelineSettings
		{
			enum
			{
				EventsPerPage = 128
			};
		};
		typedef TraceServices::TIntervalTimeline<FIoStoreActivity*, FTimelineSettings> TimelineInternal;

		explicit FIoStoreInsightsProvider(TraceServices::IAnalysisSession& Session);
		virtual ~FIoStoreInsightsProvider() = default;

		virtual void EnumerateIoStoreRequests(TFunctionRef<bool(const FIoStoreRequest&, const Timeline&)> Callback) const override;

		uint32 GetIoStoreRequestIndex(uint32 ChunkIdHash, uint8 ChunkType, uint64 Offset, uint64 Size, uint32 CallstackId, uint64 PackageId, const TCHAR* PackageName, const TCHAR* ExtraTag);
		uint32 GetUnknownIoStoreRequestIndex();

		uint64 BeginIoStoreActivity(uint32 IoStoreRequestIndex, EIoStoreActivityType Type, uint32 ThreadId, uint64 BackendHandle, double Time);
		void EndIoStoreActivity(uint32 IoStoreRequestIndex, uint64 ActivityIndex, uint64 ActualSize, bool Failed, double Time);

		void AddPackageMapping(uint64 PackageId, const TCHAR* PackageName);

		void AddBackendName(uint64 BackendHandle, const TCHAR* BackendName);

		virtual const FIoStoreRequest& GetIoStoreRequest(uint32 IoStoreRequestIndex) const override;

	private:
		struct FIoStoreRequestInfoInternal
		{
			FIoStoreRequest IoStoreRequestInfo;
			TSharedPtr<TimelineInternal> ActivityTimeline;
		};

		TraceServices::IAnalysisSession& Session;
		TraceServices::TPagedArray<FIoStoreRequestInfoInternal> IoStoreRequests;
		TraceServices::TPagedArray<FIoStoreActivity> IoStoreRequestStates;
		TMap<uint64, const TCHAR*> PackageMap;
		TMap<uint64, const TCHAR*> BackendNameMap;

		TMap<uint64,TArray<uint32>> PendingPackageNameMap;
		TMap<uint64,TArray<FIoStoreActivity*>> PendingBackendNameMap;
	};

} // namespace UE::IoStoreInsights
