// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreActivityTable.h"
#include "IoStoreActivityTableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TableCellValue.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "IIoStoreInsightsProvider.h"
#include "IO/IoChunkId.h"

#define LOCTEXT_NAMESPACE "UE::IoStoreInsights::FIoStoreActivityTable"

namespace UE::IoStoreInsights
{
	using namespace UE::Insights;

	const FName FActivityTableColumns::ColumnRequestPackage("Package");
	const FName FActivityTableColumns::ColumnRequestOffset("Offset");
	const FName FActivityTableColumns::ColumnRequestSize("Size");
	const FName FActivityTableColumns::ColumnRequestDuration("Duration");
	const FName FActivityTableColumns::ColumnRequestChunkId("ChunkId");
	const FName FActivityTableColumns::ColumnRequestChunkType("ChunkType");
	const FName FActivityTableColumns::ColumnRequestStartTime("StartTime");
	const FName FActivityTableColumns::ColumnRequestBackend("Backend");

	typedef FTableCellValue (*FActivityFieldGetterFn) (const FTableColumn&, const FIoStoreActivity*);

	template<FActivityFieldGetterFn Getter>
	class FActivityColumnValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			if (Node.IsGroup())
			{
				const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
				if (NodePtr.HasAggregatedValue(Column.GetId()))
				{
					return NodePtr.GetAggregatedValue(Column.GetId());
				}
			}
			else //if (Node->Is<FIoStoreActivityNode>())
			{
				const FIoStoreActivityNode& ActivityNode = static_cast<const FIoStoreActivityNode&>(Node);
				const FIoStoreActivity* Activity = ActivityNode.GetActivity();
				if (Activity)
				{
					return Getter(Column, Activity);
				}
			}

			return TOptional<FTableCellValue>();
		}
	};

	const TCHAR* GetActivityDisplayName(const FIoStoreActivity* Activity)
	{
		if (*Activity->IoStoreRequest->PackageName)
		{
			return Activity->IoStoreRequest->PackageName;
		}
		else if (*Activity->IoStoreRequest->ExtraTag)
		{
			return Activity->IoStoreRequest->ExtraTag;
		}
		else
		{
			return TEXT("(Unknown Package)");
		}
	}


	struct DefaultActivityFieldGetterFuncs
	{
		static FTableCellValue GetPackage  (const FTableColumn& Column, const FIoStoreActivity* Activity) { return FTableCellValue(GetActivityDisplayName(Activity)); }
		static FTableCellValue GetOffset   (const FTableColumn& Column, const FIoStoreActivity* Activity) { return FTableCellValue((int64)Activity->IoStoreRequest->Offset); }
		static FTableCellValue GetSize     (const FTableColumn& Column, const FIoStoreActivity* Activity) { return FTableCellValue((int64)Activity->ActualSize); }
		static FTableCellValue GetDuration (const FTableColumn& Column, const FIoStoreActivity* Activity) { return FTableCellValue(Activity->EndTime - Activity->StartTime); }
		static FTableCellValue GetChunkId  (const FTableColumn& Column, const FIoStoreActivity* Activity) { return FTableCellValue((int64)Activity->IoStoreRequest->ChunkIdHash); }
		static FTableCellValue GetChunkType(const FTableColumn& Column, const FIoStoreActivity* Activity) { return FTableCellValue(FText::FromString(LexToString((EIoChunkType)Activity->IoStoreRequest->ChunkType))); }
		static FTableCellValue GetStartTime(const FTableColumn& Column, const FIoStoreActivity* Activity) { return FTableCellValue(Activity->StartTime); }
		static FTableCellValue GetBackend  (const FTableColumn& Column, const FIoStoreActivity* Activity) { return FTableCellValue(Activity->BackendName); }
	};



	void FIoStoreActivityTable::Reset()
	{
		FTable::Reset();
		AddDefaultColumns();
	}

	void FIoStoreActivityTable::AddDefaultColumns()
	{
		using namespace UE::Insights;

		// Hierarchy Column (special case)
		{
			AddHierarchyColumn(-1, nullptr);
			const TSharedRef<FTableColumn>& ColumnRef = GetColumns()[0];
			ColumnRef->SetInitialWidth(75.0f);
			ColumnRef->SetShortName(LOCTEXT("ActivityColumnName",   "Hierarchy"));
			ColumnRef->SetTitleName(LOCTEXT("ActivityColumnTitle",  "All Activities"));
			ColumnRef->SetDescription(LOCTEXT("ActivityColumnDesc", "Hierarchy of all activities"));
		}

		// helper to add common column data
		int32 ColumnIndex = 0;
		auto MakeColumn = [this, &ColumnIndex]( FName InColumnName, const FText& InShortName, const FText& InTitleName, const FText& InDescription, float InWidth )
		{
			TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(InColumnName);
			AddColumn(ColumnRef);

			FTableColumn& Column = *ColumnRef;
			Column.SetIndex(ColumnIndex++);
			Column.SetShortName(InShortName);
			Column.SetTitleName(InTitleName);
			Column.SetDescription(InDescription);
			Column.SetHorizontalAlignment(HAlign_Left);
			Column.SetInitialWidth(InWidth);
			return ColumnRef;
		};

		// Package Column
		{
			TSharedRef<FTableColumn> ColumnRef = MakeColumn(FActivityTableColumns::ColumnRequestPackage,
															LOCTEXT("PackageColumnName",  "Package"),
															LOCTEXT("PackageColumnTitle", "Package"),
															LOCTEXT("PackageColumnDesc",  "Package Name/Tag being read"),
															300.0f);
			FTableColumn& Column = *ColumnRef;
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);
			Column.SetDataType(ETableCellDataType::CString);
			Column.SetValueGetter(MakeShared<FActivityColumnValueGetter<DefaultActivityFieldGetterFuncs::GetPackage>>());
			Column.SetValueFormatter(MakeShared<FCStringValueFormatterAsText>());
			Column.SetValueSorter(MakeShared<FSorterByCStringValue>(ColumnRef));
			Column.SetAggregation(ETableColumnAggregation::SameValue);
		}

		// Offset Column
		{
			TSharedRef<FTableColumn> ColumnRef = MakeColumn(FActivityTableColumns::ColumnRequestOffset,
															LOCTEXT("OffsetColumnName",  "Offset"),
															LOCTEXT("OffsetColumnTitle", "Read Offset"),
															LOCTEXT("OffsetColumnDesc",  "Offset into chunk that was requested"),
															50.0f);
			FTableColumn& Column = *ColumnRef;
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);
			Column.SetDataType(ETableCellDataType::Int64);
			Column.SetValueGetter(MakeShared<FActivityColumnValueGetter<DefaultActivityFieldGetterFuncs::GetOffset>>());
			Column.SetValueFormatter(MakeShared<FInt64ValueFormatterAsNumber>());
			Column.SetValueSorter(MakeShared<FSorterByInt64Value>(ColumnRef));
			Column.SetAggregation(ETableColumnAggregation::None);
		}

		// Size Column
		{
			TSharedRef<FTableColumn> ColumnRef = MakeColumn(FActivityTableColumns::ColumnRequestSize,
															LOCTEXT("SizeColumnName",  "Size"),
															LOCTEXT("SizeColumnTitle", "Read Size"),
															LOCTEXT("SizeColumnDesc",  "Size of the data that was returned by the IoDispatcher"),
															50.0f);
			FTableColumn& Column = *ColumnRef;
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);
			Column.SetDataType(ETableCellDataType::Int64);
			Column.SetValueGetter(MakeShared<FActivityColumnValueGetter<DefaultActivityFieldGetterFuncs::GetSize>>());
			Column.SetValueFormatter(MakeShared<FInt64ValueFormatterAsMemory>());
			Column.SetValueSorter(MakeShared<FSorterByInt64Value>(ColumnRef));
			Column.SetAggregation(ETableColumnAggregation::Sum);
		}

		// Duration Column
		{
			TSharedRef<FTableColumn> ColumnRef = MakeColumn(FActivityTableColumns::ColumnRequestDuration,
															LOCTEXT("DurationColumnName",  "Duration"),
															LOCTEXT("DurationColumnTitle", "Request Duration"),
															LOCTEXT("DurationColumnDesc",  "How long the request took to complete"),
															50.0f);
			FTableColumn& Column = *ColumnRef;
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);
			Column.SetDataType(ETableCellDataType::Double);
			Column.SetValueGetter(MakeShared<FActivityColumnValueGetter<DefaultActivityFieldGetterFuncs::GetDuration>>());
			Column.SetValueFormatter(MakeShared<FDoubleValueFormatterAsTimeAuto>());
			Column.SetValueSorter(MakeShared<FSorterByDoubleValue>(ColumnRef));
			Column.SetAggregation(ETableColumnAggregation::Sum);
		}

		// ChunkId Column
		{
			TSharedRef<FTableColumn> ColumnRef = MakeColumn(FActivityTableColumns::ColumnRequestChunkId,
															LOCTEXT("ChunkIdColumnName",  "ChunkId"),
															LOCTEXT("ChunkIdColumnTitle", "ChunkId Hash"),
															LOCTEXT("ChunkIdColumnDesc",  "Hash of the Chunk Id"),
															50.0f);
			FTableColumn& Column = *ColumnRef;
			Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);
			Column.SetDataType(ETableCellDataType::Int64);
			Column.SetValueGetter(MakeShared<FActivityColumnValueGetter<DefaultActivityFieldGetterFuncs::GetChunkId>>());
			Column.SetValueFormatter(MakeShared<FInt64ValueFormatterAsHex32>());
			Column.SetValueSorter(MakeShared<FSorterByInt64Value>(ColumnRef));
			Column.SetAggregation(ETableColumnAggregation::SameValue); // NB. non-CString SameValue aggregation currently not supported
		}

		// Chunk Type Column
		{
			TSharedRef<FTableColumn> ColumnRef = MakeColumn(FActivityTableColumns::ColumnRequestChunkType,
															LOCTEXT("ChunkTypeName",  "Chunk Type"),
															LOCTEXT("ChunkTypeTitle", "Chunk Type"),
															LOCTEXT("ChunkTypeDesc",  "The type of chunk that was requested"),
															50.0f);
			FTableColumn& Column = *ColumnRef;
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);
			Column.SetDataType(ETableCellDataType::Text);
			Column.SetValueGetter(MakeShared<FActivityColumnValueGetter<DefaultActivityFieldGetterFuncs::GetChunkType>>());
			Column.SetValueFormatter(MakeShared<FTextValueFormatter>()); // NB. using FText not CString because EIoChunkType LexToString returns FString not TCHAR*
			Column.SetValueSorter(MakeShared<FSorterByTextValue>(ColumnRef));
			Column.SetAggregation(ETableColumnAggregation::SameValue);
		}

		// Start Time Column
		{
			TSharedRef<FTableColumn> ColumnRef = MakeColumn(FActivityTableColumns::ColumnRequestStartTime,
															LOCTEXT("StartTimeName",  "Start Time"),
															LOCTEXT("StartTimeTitle", "Start Time"),
															LOCTEXT("StartTimeDesc",  "Time the request was started"),
															50.0f);
			FTableColumn& Column = *ColumnRef;
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);
			Column.SetDataType(ETableCellDataType::Double);
			Column.SetValueGetter(MakeShared<FActivityColumnValueGetter<DefaultActivityFieldGetterFuncs::GetStartTime>>());
			Column.SetValueFormatter(MakeShared<FDoubleValueFormatterAsTimeAuto>());
			Column.SetValueSorter(MakeShared<FSorterByDoubleValue>(ColumnRef));
			Column.SetAggregation(ETableColumnAggregation::Min);
		}

		// Backend Column
		{
			TSharedRef<FTableColumn> ColumnRef = MakeColumn(FActivityTableColumns::ColumnRequestBackend,
															LOCTEXT("BackendName",  "Backend"),
															LOCTEXT("BackendTitle", "Backend"),
															LOCTEXT("BackendDesc",  "IoDispatcher Backend that handled the request"),
															75.0f);
			FTableColumn& Column = *ColumnRef;
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);
			Column.SetDataType(ETableCellDataType::CString);
			Column.SetValueGetter(MakeShared<FActivityColumnValueGetter<DefaultActivityFieldGetterFuncs::GetBackend>>());
			Column.SetValueFormatter(MakeShared<FCStringValueFormatterAsText>());
			Column.SetValueSorter(MakeShared<FSorterByCStringValue>(ColumnRef));
			Column.SetAggregation(ETableColumnAggregation::SameValue);
		}
	}

} //namespace UE::IoStoreInsights

#undef LOCTEXT_NAMESPACE
