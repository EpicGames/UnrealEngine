// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "UObject/NameTypes.h"
#include "InsightsCore/Table/ViewModels/Table.h"

namespace UE::Insights { class FTableColumn; }

namespace UE::IoStoreInsights
{
	struct FIoStoreActivity;

	struct FActivityTableColumns
	{
		static const FName ColumnRequestPackage;
		static const FName ColumnRequestOffset;
		static const FName ColumnRequestSize;
		static const FName ColumnRequestDuration;
		static const FName ColumnRequestChunkId;
		static const FName ColumnRequestChunkType;
		static const FName ColumnRequestStartTime;
		static const FName ColumnRequestBackend;
	};

	class FIoStoreActivityTable : public UE::Insights::FTable
	{
	public:
		virtual ~FIoStoreActivityTable() = default;

		virtual void Reset() override;

		TArray<const FIoStoreActivity*>& GetActivities() { return Activities; }
		const TArray<const FIoStoreActivity*>& GetActivities() const { return Activities; }

		bool IsValidRowIndex(int32 InIndex) const { return Activities.IsValidIndex(InIndex); }
		const FIoStoreActivity* GetActivity(int32 InIndex) const { return IsValidRowIndex(InIndex) ? Activities[InIndex] : nullptr; }
		const FIoStoreActivity* GetActivityChecked(int32 InIndex) const { check(IsValidRowIndex(InIndex)); return Activities[InIndex]; }

	private:
		void AddDefaultColumns();

		TArray<const FIoStoreActivity*> Activities;
	};


} //namespace UE::IoStoreInsights
