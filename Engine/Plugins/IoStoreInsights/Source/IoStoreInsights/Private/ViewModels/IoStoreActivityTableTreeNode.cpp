// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreActivityTableTreeNode.h"

namespace UE::IoStoreInsights
{

INSIGHTS_IMPLEMENT_RTTI(FIoStoreActivityNode)

const FText FIoStoreActivityNode::GetDisplayName() const
{
	return FText::FromString(FString::Printf(TEXT("request_%u"), RequestIndex));
}

} // UE::IoStoreInsights
