// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ViewModels/IoStoreActivityTable.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/Table.h"

namespace UE::IoStoreInsights
{
	struct FIoStoreActivity;
	class FIoStoreActivityNode;

	// Type definition for shared pointers to instances of FIoStoreActivityNode
	typedef TSharedPtr<class FIoStoreActivityNode> FActivityNodePtr;

	// Type definition for shared references to instances of FIoStoreActivityNode
	typedef TSharedRef<class FIoStoreActivityNode> FActivityNodeRef;

	// Type definition for shared references to const instances of FIoStoreActivityNode
	typedef TSharedRef<const class FIoStoreActivityNode> FActivityNodeRefConst;

	// Type definition for weak references to instances of FIoStoreActivityNode
	typedef TWeakPtr<class FIoStoreActivityNode> FActivityNodeWeak;

	// Class used to store information about an IoStore activity (used in SActivityTreeView)
	class FIoStoreActivityNode : public UE::Insights::FTableTreeNode
	{
		INSIGHTS_DECLARE_RTTI(FIoStoreActivityNode, UE::Insights::FTableTreeNode)

	public:
		explicit FIoStoreActivityNode(TWeakPtr<FIoStoreActivityTable> InParentTable, int32 InRowIndex, uint32 InRequestIndex)
			: UE::Insights::FTableTreeNode(InParentTable, InRowIndex)
			, RequestIndex(InRequestIndex)
		{
		}

		virtual const FText GetDisplayName() const override;

		FIoStoreActivityTable& GetActivityTableChecked() const
		{
			const TSharedPtr<UE::Insights::FTable>& TablePin = GetParentTable().Pin();
			check(TablePin.IsValid());
			return *StaticCastSharedPtr<FIoStoreActivityTable>(TablePin);
		}

		bool IsValidActivity() const { return GetActivityTableChecked().IsValidRowIndex(GetRowIndex()); }
		const FIoStoreActivity* GetActivity() const { return GetActivityTableChecked().GetActivity(GetRowIndex()); }
		const FIoStoreActivity* GetActivityChecked() const { return GetActivityTableChecked().GetActivityChecked(GetRowIndex()); }

	private:
		uint32 RequestIndex;
	};

} //UE::IoStoreInsights
