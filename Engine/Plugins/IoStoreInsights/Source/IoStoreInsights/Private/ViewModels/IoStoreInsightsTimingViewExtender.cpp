// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/IoStoreInsightsTimingViewExtender.h"
#include "ViewModels/IoStoreInsightsTrack.h"
#include "Model/IoStoreInsightsProvider.h"
#include "Widgets/SIoStoreAnalysisTab.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/TimingProfilerCommon.h"
#include "IoStoreInsightsModule.h"

#define LOCTEXT_NAMESPACE "IoStoreViewExtender"

namespace UE::IoStoreInsights
{
	const uint32 FIoStoreInsightsViewSharedState::MaxLanes = 10000;



	void FIoStoreInsightsViewSharedState::Tick(const TraceServices::IAnalysisSession& InAnalysisSession)
	{
		// cache session
		if (AnalysisSession != &InAnalysisSession)
		{
			AnalysisSession = &InAnalysisSession;
			bForceIoEventsUpdate = true;
		}

		// see if we should rebuild the shared state
		if (bForceIoEventsUpdate)
		{
			bForceIoEventsUpdate = false;

			Insights::FStopwatch Stopwatch;

			// Enumerate all IO events and cache them.
			Stopwatch.Start();
			AllIoEvents.Reset();
			TArray<TSharedPtr<FIoStoreRequestState>> RequestStates;
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(InAnalysisSession);
				const IIoStoreInsightsProvider* IoStoreActivityProvider = InAnalysisSession.ReadProvider<IIoStoreInsightsProvider>(FIoStoreInsightsProvider::ProviderName);

				IoStoreActivityProvider->EnumerateIoStoreRequests([this, IoStoreActivityProvider, &RequestStates](const FIoStoreRequest& IoStoreRequest, const IIoStoreInsightsProvider::Timeline& Timeline)
				{
					TSharedPtr<FIoStoreRequestState> RequestState = RequestStates.Add_GetRef(MakeShared<FIoStoreRequestState>());
					RequestState->IoStoreRequestIndex = IoStoreRequest.IoStoreRequestIndex;
					RequestState->StartTime           = +std::numeric_limits<double>::infinity();
					RequestState->EndTime             = -std::numeric_limits<double>::infinity();
					RequestState->MaxConcurrentEvents = 0;
					RequestState->StartingDepth       = 0;

					TArray<double> ConcurrentEvents;
					Timeline.EnumerateEvents(-std::numeric_limits<double>::infinity(), +std::numeric_limits<double>::infinity(),
						[this, &RequestState, &IoStoreRequest, &Timeline, &ConcurrentEvents, IoStoreActivityProvider](double EventStartTime, double EventEndTime, uint32 EventDepth, const FIoStoreActivity* IoStoreActivity)
						{
							// view is easier to read with just read events, but some people may want to see how long a request has been waiting for
							if (bShowOnlyReadEvents && IoStoreActivity->ActivityType != EIoStoreActivityType::Request_Read)
							{
								return TraceServices::EEventEnumerate::Continue;
							}

							// events should be ordered by start time, but RequestState->StartTime may not be initialized
							ensure(RequestState->StartTime == +std::numeric_limits<double>::infinity() || EventStartTime >= RequestState->StartTime);
							if (EventStartTime < RequestState->StartTime)
							{
								RequestState->StartTime = EventStartTime;
							}
							if (EventEndTime > RequestState->EndTime)
							{
								RequestState->EndTime = EventEndTime;
							}

							uint32 LocalDepth = MAX_uint32;
							for (int32 i = 0; i < ConcurrentEvents.Num(); ++i)
							{
								if (EventStartTime >= ConcurrentEvents[i])
								{
									LocalDepth = i;
									ConcurrentEvents[i] = EventEndTime;
									break;
								}
							}
							if (LocalDepth == MAX_uint32)
							{
								LocalDepth = ConcurrentEvents.Num();
								ConcurrentEvents.Add(EventEndTime);
								RequestState->MaxConcurrentEvents = ConcurrentEvents.Num();
							}


							// slightly hacky: This becomes FIoStoreTimingEvent::Type so you can double-click on an event and see every IoStore event related to the same pacakge/bulk data (via FIoStoreInsightsTrack::BuildFilteredDrawState)
							uint64 TimingEventType = HashCombine( PointerHash(IoStoreRequest.PackageName), PointerHash(IoStoreRequest.ExtraTag));
							TimingEventType = HashCombine(TimingEventType, IoStoreRequest.PackageId);

							uint32 Type = ((uint32)IoStoreActivity->ActivityType & 0x0F) | (IoStoreActivity->Failed ? 0x80 : 0);
							AllIoEvents.Add(FIoStoreEventState{ RequestState, EventStartTime, EventEndTime, IoStoreActivity->ActualSize, IoStoreActivity->BackendName, TimingEventType, LocalDepth, Type });
							return TraceServices::EEventEnumerate::Continue;
						});

					return true;
				});
			}
			Stopwatch.Stop();
			double EnumeratedTime = Stopwatch.GetAccumulatedTime();
			if (EnumeratedTime > 0.01)
			{
				UE_LOGF(LogTimingProfiler, Verbose, "[IO] Enumerated IoStore activities (%d request states, %d events) in %ls.", RequestStates.Num(), AllIoEvents.Num(), *Insights::FormatTimeAuto(EnumeratedTime));
			}

			// Sort cached items by Start Time
			Stopwatch.Restart();
			RequestStates.Sort([](const TSharedPtr<FIoStoreRequestState>& A, const TSharedPtr<FIoStoreRequestState>& B) { return A->StartTime < B->StartTime; });
			AllIoEvents.Sort([](const FIoStoreEventState& A, const FIoStoreEventState& B) { return A.StartTime < B.StartTime; });
			Stopwatch.Stop();
			double SortTime = Stopwatch.GetAccumulatedTime();
			if (SortTime > 0.01)
			{
				UE_LOGF(LogTimingProfiler, Verbose, "[IO] Sorted IoStore activities (%d request states, %d events) in %ls.", RequestStates.Num(), AllIoEvents.Num(), *Insights::FormatTimeAuto(SortTime));
			}

			// Compute depth for IoStore activities (avoids overlaps).
			if (RequestStates.Num() > 0)
			{
				struct FLane
				{
					double EndTime = 0.0f;
				};
				TArray<FLane> Lanes; // one lane per event depth, a IoStore activity occupies multiple lanes

				Stopwatch.Restart();
				for (const TSharedPtr<FIoStoreRequestState>& IoStoreActivityPtr : RequestStates)
				{
					FIoStoreRequestState& RequestState = *IoStoreActivityPtr;

					// Find lane (avoiding overlaps with other IoStore activities).
					int32 Depth = 0;
					while (Depth < Lanes.Num())
					{
						bool bOverlap = false;
						for (int32 LocalDepth = 0; LocalDepth < RequestState.MaxConcurrentEvents; ++LocalDepth)
						{
							if (Depth + LocalDepth >= Lanes.Num())
							{
								break;
							}
							const FLane& Lane = Lanes[Depth + LocalDepth];
							if (RequestState.StartTime < Lane.EndTime)
							{
								bOverlap = true;
								Depth += LocalDepth;
								break;
							}
						}
						if (!bOverlap)
						{
							break;
						}
						++Depth;
					}

					int32 NewLaneNum = Depth + RequestState.MaxConcurrentEvents;

					if (NewLaneNum > MaxLanes)
					{
						// Snap to the bottom; allows overlaps in this case.
						RequestState.StartingDepth = MaxLanes - RequestState.MaxConcurrentEvents;
					}
					else
					{
						if (NewLaneNum > Lanes.Num())
						{
							Lanes.AddDefaulted(NewLaneNum - Lanes.Num());
						}

						RequestState.StartingDepth = Depth;
						for (int32 LocalDepth = 0; LocalDepth < RequestState.MaxConcurrentEvents; ++LocalDepth)
						{
							Lanes[Depth + LocalDepth].EndTime = RequestState.EndTime;
						}
					}
				}
				Stopwatch.Stop();
				UE_LOGF(LogTimingProfiler, Verbose, "[IO] Computed layout for IoStore activities in %ls.", *Insights::FormatTimeAuto(Stopwatch.GetAccumulatedTime()));

				Stopwatch.Restart();
				for (FIoStoreEventState& Event : AllIoEvents)
				{
					Event.Depth += Event.RequestState->StartingDepth;
					ensure(Event.Depth < MaxLanes);
				}
				Stopwatch.Stop();
				UE_LOGF(LogTimingProfiler, Verbose, "[IO] Updated depth for events in %ls.", *Insights::FormatTimeAuto(Stopwatch.GetAccumulatedTime()));
			}
		}
	}








	void FIoStoreInsightsTimingViewExtender::OnBeginSession(Insights::Timing::ITimingViewSession& InSession)
	{
		FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
		if (PerSessionData == nullptr)
		{
			PerSessionData = &PerSessionDataMap.Add(&InSession);
		}
	}



	void FIoStoreInsightsTimingViewExtender::OnEndSession(Insights::Timing::ITimingViewSession& InSession)
	{
		FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
		if (PerSessionData != nullptr)
		{
			if (PerSessionData->IoStoreActivityTrack.IsValid())
			{
				InSession.RemoveScrollableTrack(PerSessionData->IoStoreActivityTrack);
				PerSessionData->IoStoreActivityTrack.Reset();
			}

			PerSessionDataMap.Remove(&InSession);
		}

		const bool bInvoke = false;
		if (TSharedPtr<SIoStoreAnalysisTab> AnalysisView = FIoStoreInsightsModule::Get().GetIoStoreAnalysisViewTab(bInvoke))
		{
			AnalysisView->SetSession(nullptr, nullptr, nullptr);
		}
	}



	void FIoStoreInsightsTimingViewExtender::Tick(Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
	{
		bool bRequestUpdate = false;

		// periodically refresh the shared data for each view while the session is still loading
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(InAnalysisSession);
			double SessionDuration = InAnalysisSession.GetDurationSeconds();
			bool bAnalysisComplete = InAnalysisSession.IsAnalysisComplete();
			if ((bAnalysisComplete && !bWasAnalysisComplete) || (SessionDuration - PreviousAnalysisSessionDuration) > 0.25f)
			{
				PreviousAnalysisSessionDuration = SessionDuration;
				bWasAnalysisComplete = bAnalysisComplete;
				bRequestUpdate = true;
			}
		}


		FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
		if (PerSessionData != nullptr)
		{
			// create track if necessary
			if (!PerSessionData->IoStoreActivityTrack.IsValid())
			{
				PerSessionData->IoStoreActivityTrack = MakeShared<FIoStoreInsightsTrack>(SharedState);
				PerSessionData->IoStoreActivityTrack->SetOrder(FTimingTrackOrder::Last);
				PerSessionData->IoStoreActivityTrack->SetVisibilityFlag(false);
				InSession.AddScrollableTrack(PerSessionData->IoStoreActivityTrack);
			}

			// refresh the shared data if the track is dirty
			bRequestUpdate |= PerSessionData->IoStoreActivityTrack->IsDirty() && PerSessionData->IoStoreActivityTrack->IsVisible();
		}


		// see if we should rebuild the shared state
		if (bRequestUpdate)
		{
			SharedState.RequestUpdate();
		}

		SharedState.Tick(InAnalysisSession);

		// update tab
		const bool bInvoke = false;
		if (TSharedPtr<SIoStoreAnalysisTab> AnalysisView = FIoStoreInsightsModule::Get().GetIoStoreAnalysisViewTab(bInvoke))
		{
			AnalysisView->SetSession(&InSession, &InAnalysisSession, &SharedState);
		}

	}



	void FIoStoreInsightsTimingViewExtender::ExtendOtherTracksFilterMenu(Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
	{
		FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
		if (PerSessionData != nullptr)
		{
			// note: can't use FUICommandList etc. because STimingView is private
			InMenuBuilder.BeginSection("IoStore Activity", LOCTEXT("ContextMenu_Section_IoStoreActivity", "IoStore Activity"));
			{
				InMenuBuilder.AddMenuEntry(
					LOCTEXT("ContextMenu_Item_IoStoreActivityTrack", "IoStore Activity Track"),
					LOCTEXT("ContextMenu_Item_IoStoreActivityTrackTip", "Shows/hides the IoStore activity track"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw(this, &FIoStoreInsightsTimingViewExtender::ToggleActivityTrackVisibility, PerSessionData->IoStoreActivityTrack),
						FCanExecuteAction(),
						FIsActionChecked::CreateRaw(this, &FIoStoreInsightsTimingViewExtender::IsIoStoreActivityTrackVisible, PerSessionData->IoStoreActivityTrack)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);

				InMenuBuilder.AddMenuEntry(
					LOCTEXT("ContextMenu_Item_IoStoreShowAllEvents", "Only Reads (IoStore Activity Track)"),
					LOCTEXT("ContextMenu_Item_IoStoreShowAllEventsTip", "Shows/hides all IoStore events"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw(&SharedState, &FIoStoreInsightsViewSharedState::ToggleShowOnlyReadEvents),
						FCanExecuteAction(),
						FIsActionChecked::CreateRaw(&SharedState, &FIoStoreInsightsViewSharedState::IsShowingOnlyReadEvents)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);

			}
			InMenuBuilder.EndSection();

		}
	}



	bool FIoStoreInsightsTimingViewExtender::IsIoStoreActivityTrackVisible(TSharedPtr<FIoStoreInsightsTrack> IoStoreActivityTrack) const
	{
		return IoStoreActivityTrack && IoStoreActivityTrack->IsVisible();
	}



	void FIoStoreInsightsTimingViewExtender::ToggleActivityTrackVisibility(TSharedPtr<FIoStoreInsightsTrack> IoStoreActivityTrack)
	{
		if (IoStoreActivityTrack.IsValid())
		{
			IoStoreActivityTrack->ToggleVisibility();
		}

		const bool bIsActivityTrackVisible = IsIoStoreActivityTrackVisible(IoStoreActivityTrack);
		if (bIsActivityTrackVisible)
		{
			SharedState.RequestUpdate();
		}
	}

} // namespace UE::IoStoreInsights

#undef LOCTEXT_NAMESPACE
