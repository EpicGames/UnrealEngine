// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Insights/ITimingViewExtender.h"

namespace TraceServices
{ 
	class IAnalysisSession; 
}

namespace UE::Insights::Timing 
{ 
	class ITimingViewSession; 
}

namespace UE::IoStoreInsights
{
	class FIoStoreInsightsTrack;

	struct FIoStoreRequestState
	{
		uint32 IoStoreRequestIndex;
		double StartTime;
		double EndTime;
		int32 MaxConcurrentEvents; // only used during layout
		uint32 StartingDepth; // only used during layout
	};

	struct FIoStoreEventState
	{
		TSharedPtr<FIoStoreRequestState> RequestState;
		double StartTime;
		double EndTime;
		uint64 ActualSize;
		const TCHAR* BackendName;
		uint64 TimingEventType;
		uint32 Depth; // During update, this is local within a track - then it's set to a global depth
		uint32 Type; // EIoStoreActivityType + "Failed" flag
	};

	class FIoStoreInsightsViewSharedState
	{
	public:
		void Tick(const TraceServices::IAnalysisSession& InAnalysisSession);

		const TraceServices::IAnalysisSession& GetAnalysisSession() const { return *AnalysisSession; }
		const TArray<FIoStoreEventState>& GetAllEvents() const { return AllIoEvents; }

		void RequestUpdate() { bForceIoEventsUpdate = true; }

		bool IsShowingOnlyReadEvents() const { return bShowOnlyReadEvents; }
		void ToggleShowOnlyReadEvents() { bShowOnlyReadEvents = !bShowOnlyReadEvents; RequestUpdate(); }

		static const uint32 MaxLanes;

	private:
		const TraceServices::IAnalysisSession* AnalysisSession = nullptr; // cached in Tick()
		bool bForceIoEventsUpdate = false;
		bool bShowOnlyReadEvents = true;
		TArray<FIoStoreEventState> AllIoEvents;
	};

	class FIoStoreInsightsTimingViewExtender : public Insights::Timing::ITimingViewExtender
	{
	public:
		virtual void OnBeginSession(Insights::Timing::ITimingViewSession& InSession) override;
		virtual void OnEndSession(Insights::Timing::ITimingViewSession& InSession) override;
		virtual void Tick(Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
		virtual void ExtendOtherTracksFilterMenu(Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;

	private:
		bool IsIoStoreActivityTrackVisible(TSharedPtr<FIoStoreInsightsTrack> IoStoreActivityTrack) const;
		void ToggleActivityTrackVisibility(TSharedPtr<FIoStoreInsightsTrack> IoStoreActivityTrack);


		bool bWasAnalysisComplete = false;
		double PreviousAnalysisSessionDuration = 0.0f;
		FIoStoreInsightsViewSharedState SharedState;

		struct FPerSessionData
		{
			TSharedPtr<FIoStoreInsightsTrack> IoStoreActivityTrack;

		};
		TMap<Insights::Timing::ITimingViewSession*, FPerSessionData> PerSessionDataMap;


	};
} // namespace UE::IoStoreInsights
