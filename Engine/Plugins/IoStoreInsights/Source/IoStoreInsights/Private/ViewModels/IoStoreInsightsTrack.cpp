// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/IoStoreInsightsTrack.h"
#include "ViewModels/IoStoreInsightsTimingViewExtender.h"
#include "Model/IoStoreInsightsProvider.h"

#include "Algo/BinarySearch.h"
#include "TraceServices/Model/Callstack.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "IO/IoChunkId.h"

namespace UE::IoStoreInsights
{
	class FIoStoreTimingEvent : public FTimingEvent
	{
	public:
		FIoStoreTimingEvent(const TSharedRef<const FIoStoreInsightsTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth, const FIoStoreEventState& InEventState)
			: FTimingEvent(InTrack, InStartTime, InEndTime, InDepth, InEventState.TimingEventType) {}
		INSIGHTS_DECLARE_RTTI(FIoStoreTimingEvent, FTimingEvent)
	};
	INSIGHTS_IMPLEMENT_RTTI(FIoStoreTimingEvent)





	FIoStoreInsightsTrack::FIoStoreInsightsTrack(FIoStoreInsightsViewSharedState& InSharedState)
		: FTimingEventsTrack(TEXT("IoStore Activity"))
		, SharedState(InSharedState)
	{
	}


	void FIoStoreInsightsTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
	{
		InOutTooltip.ResetContent();

		if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FIoStoreTimingEvent>())
		{
			const FIoStoreTimingEvent& TooltipEvent = InTooltipEvent.As<FIoStoreTimingEvent>();

			auto MatchEvent = [this, &TooltipEvent](double InStartTime, double InEndTime, uint32 InDepth)
			{
				return InDepth == TooltipEvent.GetDepth()
					&& InStartTime == TooltipEvent.GetStartTime()
					&& InEndTime == TooltipEvent.GetEndTime();
			};

			FTimingEventSearchParameters SearchParameters(TooltipEvent.GetStartTime(), TooltipEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch, MatchEvent);
			FindIoStoreEvent(SearchParameters, [this, &InOutTooltip, &TooltipEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FIoStoreEventState& InEvent)
			{
				const IIoStoreInsightsProvider* IoStoreActivityProvider = SharedState.GetAnalysisSession().ReadProvider<IIoStoreInsightsProvider>(FIoStoreInsightsProvider::ProviderName);
				const FIoStoreRequest& IoStoreRequest = IoStoreActivityProvider->GetIoStoreRequest(InEvent.RequestState->IoStoreRequestIndex);
				const EIoStoreActivityType ActivityType = static_cast<EIoStoreActivityType>(InEvent.Type & 0x0F);
				const bool bHasFailed = ((InEvent.Type & 0xF0) != 0);

				// prepare tooltip title
				FString TypeStr;
				uint32 TypeColor;
				if (bHasFailed)
				{
					TypeStr = TEXT("Failed ");
					TypeStr += LexToString(ActivityType);
					TypeColor = 0xFFFF3333;
				}
				else
				{
					TypeStr = LexToString(ActivityType);
					TypeColor = GetIoStoreActivityTypeColor(ActivityType);
				}
				FLinearColor TypeLinearColor = FLinearColor(FColor(TypeColor));
				TypeLinearColor.R *= 2.0f;
				TypeLinearColor.G *= 2.0f;
				TypeLinearColor.B *= 2.0f;
				InOutTooltip.AddTitle(TypeStr, TypeLinearColor);

				// add tooltip fields
				const double Duration = InEvent.EndTime - InEvent.StartTime;
				InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), Insights::FormatTimeAuto(Duration));

				if (*IoStoreRequest.PackageName)
				{
					InOutTooltip.AddNameValueTextLine(TEXT("Package:"), IoStoreRequest.PackageName);
				}
				if (*IoStoreRequest.ExtraTag)
				{
					InOutTooltip.AddNameValueTextLine(TEXT("Tag:"), IoStoreRequest.ExtraTag);
				}
				InOutTooltip.AddNameValueTextLine(TEXT("Chunk Type:"), LexToString((EIoChunkType)IoStoreRequest.ChunkType) );
				InOutTooltip.AddNameValueTextLine(TEXT("Chunk Id Hash:"), *FString::Printf(TEXT("0x%X"), IoStoreRequest.ChunkIdHash ) );
				InOutTooltip.AddNameValueTextLine(TEXT("Offset:"), FText::AsNumber(IoStoreRequest.Offset).ToString() + TEXT(" bytes"));
				if (ActivityType == EIoStoreActivityType::Request_Read)
				{
					if (IoStoreRequest.Size != std::numeric_limits<uint64>::max() && IoStoreRequest.Size != InEvent.ActualSize)
					{
						InOutTooltip.AddNameValueTextLine(TEXT("Size:"), FText::AsNumber(IoStoreRequest.Size).ToString() + TEXT(" bytes"));
					}
					InOutTooltip.AddNameValueTextLine(TEXT("Result Size:"), FText::AsNumber(InEvent.ActualSize).ToString() + TEXT(" bytes"));
					InOutTooltip.AddNameValueTextLine(TEXT("Backend:"), InEvent.BackendName);
				}
				else
				{
					InOutTooltip.AddNameValueTextLine(TEXT("Requested Size:"), IoStoreRequest.Size != std::numeric_limits<uint64>::max() ? (FText::AsNumber(IoStoreRequest.Size).ToString() + TEXT(" bytes")) : TEXT("(all available data)") );
				}

				// potentially add a callstack at the very bottom
				static const bool bAlwaysShowCallstack = false; // @todo: put on the right-click menu
				if (bAlwaysShowCallstack || (*IoStoreRequest.PackageName == '\0' && *IoStoreRequest.ExtraTag == '\0')) // only show callstack if there's no package or custom tag
				{
					// package id may be useful for debugging where the missing tag is
					if (IoStoreRequest.PackageId != 0)
					{
						InOutTooltip.AddNameValueTextLine(TEXT("Package Id:"), FString::Printf(TEXT("0x%llX"), IoStoreRequest.PackageId)); 
					}

					// append the callstack
					const TraceServices::ICallstacksProvider* CallstacksProvider = SharedState.GetAnalysisSession().ReadProvider<TraceServices::ICallstacksProvider>(TraceServices::GetCallstacksProviderName());
					const TraceServices::FCallstack* Callstack = (IoStoreRequest.CallstackId && CallstacksProvider) ? CallstacksProvider->GetCallstack(IoStoreRequest.CallstackId) : nullptr;
					if (Callstack)
					{
						InOutTooltip.AddTextLine(TEXT(""), FTooltipDrawState::DefaultNameColor);
						for (uint8 CallstackFrame = 0; CallstackFrame < Callstack->Num(); CallstackFrame++)
						{
							InOutTooltip.AddTextLine(Callstack->Name(CallstackFrame), FTooltipDrawState::DefaultNameColor);
						}
					}
				}

				InOutTooltip.UpdateLayout();
			});
		}
	}



	bool FIoStoreInsightsTrack::FindIoStoreEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FIoStoreEventState&)> InFoundPredicate) const
	{
		return TTimingEventSearch<FIoStoreEventState>::Search(
			InParameters,

			// Search...
			[this](TTimingEventSearch<FIoStoreEventState>::FContext& InContext)
			{
				const TArray<FIoStoreEventState>& Events = SharedState.GetAllEvents();

				// Events are sorted by start time.
				// Find the first event with StartTime >= searched EndTime.
				int32 StartIndex = Algo::LowerBoundBy(Events, InContext.GetParameters().EndTime,
					[](const FIoStoreEventState& Event) { return Event.StartTime; });

				// Start at the last event with StartTime < searched EndTime.
				for (int32 Index = StartIndex - 1; Index >= 0; --Index)
				{
					const FIoStoreEventState& Event = Events[Index];

					if (Event.EndTime <= InContext.GetParameters().StartTime ||
						Event.StartTime >= InContext.GetParameters().EndTime)
					{
						continue;
					}

					InContext.Check(Event.StartTime, Event.EndTime, Event.Depth, Event);

					if (!InContext.ShouldContinueSearching())
					{
						break;
					}
				}
			},

			// Found!
			[&InFoundPredicate](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FIoStoreEventState& InEvent)
			{
				InFoundPredicate(InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent);
			});
	}



	void FIoStoreInsightsTrack::AddIoStoreEventToBuilder(const FIoStoreEventState& Event, ITimingEventsTrackDrawStateBuilder& Builder, const IIoStoreInsightsProvider* IoStoreActivityProvider) const
	{
		ensure(Event.Depth < FIoStoreInsightsViewSharedState::MaxLanes);

		const EIoStoreActivityType ActivityType = static_cast<EIoStoreActivityType>(Event.Type & 0x0F);
		const bool bHasFailed = ((Event.Type & 0xF0) != 0);
		
		uint32 Color;
		if (bHasFailed)
		{
			Color = 0xFFAA0000;
		}
		else if (ActivityType == EIoStoreActivityType::Request_Read)
		{
			Color =  FTimingEvent::ComputeEventColor(Event.BackendName);
		}
		else
		{
			Color = GetIoStoreActivityTypeColor(ActivityType);
		}

		Builder.AddEvent(Event.StartTime, Event.EndTime, Event.Depth, Color,
			[&Event, ActivityType, bHasFailed, this](float Width)
			{
				FString EventName;

				if (bHasFailed)
				{
					EventName += TEXT("Failed ");
				}

				EventName += LexToString(ActivityType);

				const float MinWidth = static_cast<float>(EventName.Len()) * 4.0f + 32.0f;
				if (Width > MinWidth)
				{
					const double Duration = Event.EndTime - Event.StartTime;
					EventName += TEXT(" (");
					EventName += UE::Insights::FormatTimeAuto(Duration);
					EventName += TEXT(")");
				}

				return EventName;
			});
	}



	void FIoStoreInsightsTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
	{
		const IIoStoreInsightsProvider* IoStoreActivityProvider = SharedState.GetAnalysisSession().ReadProvider<IIoStoreInsightsProvider>(FIoStoreInsightsProvider::ProviderName);

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		for (const FIoStoreEventState& Event : SharedState.GetAllEvents())
		{
			if (Event.EndTime <= Viewport.GetStartTime())
			{
				continue;
			}
			if (Event.StartTime >= Viewport.GetEndTime())
			{
				break;
			}
			if (Event.Depth >= FIoStoreInsightsViewSharedState::MaxLanes)
			{
				continue;
			}

			AddIoStoreEventToBuilder(Event, Builder, IoStoreActivityProvider);
		}
	}



	void FIoStoreInsightsTrack::BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
	{
		const TSharedPtr<ITimingEventFilter> EventFilterPtr = Context.GetEventFilter();
		if (EventFilterPtr.IsValid() && EventFilterPtr->FilterTrack(*this))
		{
			const FTimingTrackViewport& Viewport = Context.GetViewport();
			const IIoStoreInsightsProvider* IoStoreActivityProvider = SharedState.GetAnalysisSession().ReadProvider<IIoStoreInsightsProvider>(FIoStoreInsightsProvider::ProviderName);

			for (const FIoStoreEventState& Event : SharedState.GetAllEvents())
			{
				if (Event.EndTime <= Viewport.GetStartTime())
				{
					continue;
				}
				if (Event.StartTime >= Viewport.GetEndTime())
				{
					break;
				}
				if (Event.Depth >= FIoStoreInsightsViewSharedState::MaxLanes)
				{
					continue;
				}

				FIoStoreTimingEvent TimingEvent(SharedThis(this), Event.StartTime, Event.EndTime, Event.Depth, Event);
				if (EventFilterPtr->FilterEvent(TimingEvent))
				{
					AddIoStoreEventToBuilder(Event, Builder, IoStoreActivityProvider);
				}
			}
		}
	}



	const TSharedPtr<const ITimingEvent> FIoStoreInsightsTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
	{
		TSharedPtr<const ITimingEvent> FoundEvent;

		FindIoStoreEvent(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FIoStoreEventState& InEventState)
		{
			FoundEvent = MakeShared<FIoStoreTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth, InEventState);
		});

		return FoundEvent;
	}



	uint32 FIoStoreInsightsTrack::GetIoStoreActivityTypeColor(EIoStoreActivityType Type) const
	{
		switch(Type)
		{
			case EIoStoreActivityType::Request_Pending: return 0xFF334433;
			case EIoStoreActivityType::Request_Read:    return 0xFF33AA33;
		}
		return 0x55333333;
	}

} // namespace UE::IoStoreInsights

