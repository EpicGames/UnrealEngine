// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventsTrack.h"

class FTimingEventSearchParameters;

namespace UE::IoStoreInsights
{
	enum class EIoStoreActivityType : uint8;
	class FIoStoreInsightsViewSharedState;
	struct FIoStoreEventState;

	class FIoStoreInsightsTrack : public FTimingEventsTrack
	{
	public:
		explicit FIoStoreInsightsTrack(FIoStoreInsightsViewSharedState& InSharedState);
		virtual ~FIoStoreInsightsTrack() override = default;

		virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
		virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
		virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
		virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;

	protected:
		void AddIoStoreEventToBuilder(const FIoStoreEventState& Event, ITimingEventsTrackDrawStateBuilder& Builder, const class IIoStoreInsightsProvider* IoStoreActivityProvider) const;
		bool FindIoStoreEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FIoStoreEventState&)> InFoundPredicate) const;

		uint32 GetIoStoreActivityTypeColor(EIoStoreActivityType Type) const;

	protected:
		FIoStoreInsightsViewSharedState& SharedState;
	};

} // namespace UE::IoStoreInsights
