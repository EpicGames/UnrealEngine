// Copyright Epic Games, Inc. All Rights Reserved.

#include "SIoStoreActivityTableTreeView.h"
#include "Model/IoStoreInsightsProvider.h"
#include "ViewModels/IoStoreActivityTable.h"
#include "ViewModels/IoStoreActivityTableTreeNode.h"


namespace UE::IoStoreInsights
{
	void SActivityTableTreeView::Construct(const FArguments& InArgs, TSharedPtr<FIoStoreActivityTable> InTablePtr)
	{
		ConstructWidget(InTablePtr);
		CreateGroupings();
		CreateSortings();
	}



	void SActivityTableTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		STableTreeView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		if (!bIsUpdateRunning)
		{
			RebuildTree(false);
		}
	}



	void SActivityTableTreeView::RebuildTree(bool bResync)
	{
		if (StartTime > EndTime)
		{
			bRangeDirty = false;
			return;
		}

		if (bResync || bRangeDirty)
		{
			bRangeDirty = false;

			TSharedPtr<FIoStoreActivityTable> ActivityTable = GetActivityTable();
			TArray<const FIoStoreActivity*>& Activities = ActivityTable->GetActivities();
			Activities.Empty();
			TableRowNodes.Empty();

			if (StartTime < EndTime && AnalysisSession != nullptr)
			{
				const FIoStoreInsightsProvider* Provider = AnalysisSession->ReadProvider<FIoStoreInsightsProvider>(IIoStoreInsightsProvider::ProviderName);
				if (Provider)
				{
					//TODO: thread safety!
					Provider->EnumerateIoStoreRequests(
						[this, &ActivityTable, &Activities]
						(const FIoStoreRequest& IoStoreRequest, const IIoStoreInsightsProvider::Timeline& Timeline)
						{
							Timeline.EnumerateEvents(StartTime, EndTime,
								[this, &ActivityTable, &Activities]
								(double EventStartTime, double EventEndTime, uint32 EventDepth, const FIoStoreActivity* IoStoreActivity)
								{
									if (IoStoreActivity->ActivityType == EIoStoreActivityType::Request_Read && IoStoreActivity->EndTime > 0 && !IoStoreActivity->Failed)
									{
										Activities.Emplace(IoStoreActivity);
										const uint32 TableRowIndex = Activities.Num() - 1;
										const uint32 RequestIndex = IoStoreActivity->IoStoreRequest->IoStoreRequestIndex;
										FActivityNodePtr NodePtr = MakeShared<FIoStoreActivityNode>(ActivityTable, TableRowIndex, RequestIndex);
										TableRowNodes.Add(NodePtr);
									}

									return TraceServices::EEventEnumerate::Continue;
								});

							return true;
						});
				}
			}

			UpdateTree();
			TreeView->RebuildList();
		}
	}



	void SActivityTableTreeView::SetRange(double InStartTime, double InEndTime)
	{
		bRangeDirty = (StartTime != InStartTime) || (EndTime != InEndTime);

		StartTime = InStartTime;
		EndTime = InEndTime;
	}



	void SActivityTableTreeView::SetAnalysisSession(const TraceServices::IAnalysisSession* InAnalysisSession)
	{
		AnalysisSession = InAnalysisSession;
	}



} // UE::IoStoreInsights
