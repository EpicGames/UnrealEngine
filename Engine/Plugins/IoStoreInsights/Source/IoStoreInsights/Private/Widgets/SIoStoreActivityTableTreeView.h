// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InsightsCore/Table/Widgets/STableTreeView.h"
#include "ViewModels/IoStoreActivityTable.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE::IoStoreInsights
{
	class SActivityTableTreeView : public UE::Insights::STableTreeView
	{
	public:
		virtual ~SActivityTableTreeView() = default;

		SLATE_BEGIN_ARGS(SActivityTableTreeView) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<FIoStoreActivityTable> InTablePtr);

		TSharedPtr<FIoStoreActivityTable> GetActivityTable() { return StaticCastSharedPtr<FIoStoreActivityTable>(GetTable()); }
		const TSharedPtr<FIoStoreActivityTable> GetActivityTable() const { return StaticCastSharedPtr<FIoStoreActivityTable>(GetTable()); }

		void SetRange(double InStartTime, double InEndTime);
		void SetAnalysisSession(const TraceServices::IAnalysisSession* InAnalysisSession);

		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		virtual void RebuildTree(bool bResync);

	private:
		bool bRangeDirty = false;
		double StartTime = 0.0;
		double EndTime = -1.0;
		const TraceServices::IAnalysisSession* AnalysisSession = nullptr;
	};

} //UE::IoStoreInsights
