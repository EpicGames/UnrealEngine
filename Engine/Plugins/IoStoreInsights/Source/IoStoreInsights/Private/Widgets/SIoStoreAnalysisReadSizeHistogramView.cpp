// Copyright Epic Games, Inc. All Rights Reserved.

#include "SIoStoreAnalysisReadSizeHistogramView.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "InsightsCore/Common/TimeUtils.h"

#define LOCTEXT_NAMESPACE "SIoStoreAnalysisReadSizeHistogramView"

namespace UE::IoStoreInsights
{
	namespace Private
	{
		const FName ColumnQuantizedReadSize("ReadSize");
		const FName ColumnQuantizedReadCountGraph("ReadCountGraph");
		const FName ColumnQuantizedReadCount("ReadCount");
		const FName ColumnQuantizedReadTime("ReadTime");

		struct SReadSizeHistogramViewRow : public SMultiColumnTableRow<TSharedPtr<Private::FReadSizeHistogramItem>>
		{
			SLATE_BEGIN_ARGS(SReadSizeHistogramViewRow) {}
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<Private::FReadSizeHistogramItem> InItem)
			{
				Item = InItem;
				SMultiColumnTableRow<TSharedPtr<Private::FReadSizeHistogramItem>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
			}

			TSharedRef<SWidget> GenerateWidgetForColumn(const FName& Column)
			{
				if (Column == Private::ColumnQuantizedReadSize)
				{
					return SNew(STextBlock)
						.Text(FText::AsMemory(Item->QuantizedReadSize));
				}
				if (Column == Private::ColumnQuantizedReadCountGraph)
				{
					return SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.FillWidth(0.1f)
						[
							SNew(STextBlock)
							.Text(FText::AsPercent(Item->CountAsPct))
							.TextStyle(FAppStyle::Get(), "TreeTable.Tooltip")
						]

						+SHorizontalBox::Slot()
						.FillWidth(0.9f)
						[
							SNew(SProgressBar)
								.Percent(Item->CountAsPctNormalized)
								.BorderPadding(FVector2D(4, 4))
								.BarFillStyle(EProgressBarFillStyle::Scale)
								.BackgroundImage(FAppStyle::Get().GetBrush("NoBrush"))
								.FillImage(FAppStyle::Get().GetBrush("WhiteBrush"))
						];
				}
				else if (Column == Private::ColumnQuantizedReadCount)
				{
					return SNew(STextBlock)
						.Text(FText::AsNumber(Item->Count));
				}
				else if (Column == Private::ColumnQuantizedReadTime)
				{
					if (Item->Count > 0)
					{
						const double AvgDuration = (Item->MinDuration + Item->MaxDuration) / 2.0;
						return SNew(STextBlock)
							.Text(FText::FromString(*UE::Insights::FormatTimeAuto(AvgDuration)));
					}
					else
					{
						return SNew(STextBlock)
							.Text(LOCTEXT("NotApplicable","N/A"));
					}
				}

				return SNullWidget::NullWidget;
			}

		private:
			TSharedPtr<Private::FReadSizeHistogramItem> Item;
		};


	}


	void SIoStoreAnalysisReadSizeHistogramView::Construct(const FArguments& InArgs)
	{
		Super::Construct(
			Super::FArguments()
			.ListItemsSource(InArgs._ListItemsSource)
			.ScrollbarVisibility(EVisibility::Visible)
			.OnGenerateRow(this, &SIoStoreAnalysisReadSizeHistogramView::OnGenerateRow)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+SHeaderRow::Column(Private::ColumnQuantizedReadSize)
				.DefaultLabel(LOCTEXT("ColumnQuantizedReadSize", "Quantized Size"))
				.FillWidth(0.20f)

				+SHeaderRow::Column(Private::ColumnQuantizedReadCountGraph)
				.DefaultLabel(LOCTEXT("ColumnQuantizedReadGraph", "Histogram"))
				.FillWidth(0.45f)

				+SHeaderRow::Column(Private::ColumnQuantizedReadCount)
				.DefaultLabel(LOCTEXT("ColumnQuantizedReadCount", "Num. Reads"))
				.FillWidth(0.15f)

				+SHeaderRow::Column(Private::ColumnQuantizedReadTime)
				.DefaultLabel(LOCTEXT("ColumnQuantizedReadTime", "Avg. Duration"))
				.FillWidth(0.20f)						
			)
		);
	}


	TSharedRef<ITableRow> SIoStoreAnalysisReadSizeHistogramView::OnGenerateRow(TSharedPtr<Private::FReadSizeHistogramItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(Private::SReadSizeHistogramViewRow, OwnerTable, Item);
	}

	

	void SIoStoreAnalysisReadSizeHistogramView::SetAnalysisSession( const TraceServices::IAnalysisSession* InAnalysisSession )
	{
		AnalysisSession = InAnalysisSession;
	}

} //namespace UE::IoStoreInsights

#undef LOCTEXT_NAMESPACE
