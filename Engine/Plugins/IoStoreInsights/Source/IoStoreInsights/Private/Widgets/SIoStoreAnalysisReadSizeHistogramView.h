// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::IoStoreInsights
{
	namespace Private
	{ 
		struct FReadSizeHistogramItem
		{
			uint64 QuantizedReadSize = 0;
			uint32 Count = 0;
			float CountAsPct = 0;
			float CountAsPctNormalized = 0;
			double MinDuration = 0;
			double MaxDuration = 0;
		};
	}

	class SIoStoreAnalysisReadSizeHistogramView : public SListView<TSharedPtr<Private::FReadSizeHistogramItem>>
	{
		typedef SListView<TSharedPtr<Private::FReadSizeHistogramItem>> Super;

	public:
		SLATE_BEGIN_ARGS(SIoStoreAnalysisReadSizeHistogramView){}
			SLATE_ARGUMENT(TArray<TSharedPtr<Private::FReadSizeHistogramItem>>*, ListItemsSource)
		SLATE_END_ARGS()
		~SIoStoreAnalysisReadSizeHistogramView() = default;

		void Construct(const FArguments& InArgs);
		void SetAnalysisSession( const TraceServices::IAnalysisSession* InAnalysisSession );

	private:
		const TraceServices::IAnalysisSession* AnalysisSession = nullptr;

		TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<Private::FReadSizeHistogramItem> Item, const TSharedRef<STableViewBase>& OwnerTable);
	};

} //namespace UE::IoStoreInsights

