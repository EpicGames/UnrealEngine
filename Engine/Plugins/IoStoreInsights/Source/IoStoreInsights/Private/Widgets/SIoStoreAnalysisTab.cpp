// Copyright Epic Games, Inc. All Rights Reserved.

#include "SIoStoreAnalysisTab.h"
#include "SIoStoreAnalysisReadSizeHistogramView.h"
#include "SIoStoreActivityTableTreeView.h"
#include "Model/IoStoreInsightsProvider.h"
#include "ViewModels/IoStoreInsightsTrack.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SCheckBox.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/ViewModels/ITimingEvent.h"
#include "IO/IoChunkId.h"

#define LOCTEXT_NAMESPACE "SIoStoreAnalysisTab"

namespace UE::IoStoreInsights
{
	void SIoStoreAnalysisTab::Construct(const FArguments& InArgs)
	{
		// create main widgets
		TSharedPtr<SHorizontalBox> ViewModeSelectorsBox;
		TSharedPtr<SWidgetSwitcher> ViewModeSwitcher;
		ChildSlot
		[
			SNew(SVerticalBox)

			// view mode selection
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8,2)
			[
				SAssignNew(ViewModeSelectorsBox, SHorizontalBox)
			]

			// active view mode panel
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(3)
			[
				SAssignNew(ViewModeSwitcher, SWidgetSwitcher)
				.WidgetIndex(this, &SIoStoreAnalysisTab::GetSelectedViewModeIndex)
			]
		];

		// helper to create widgets for a view mode
		auto AddViewMode = [this, &ViewModeSelectorsBox, &ViewModeSwitcher]( EViewMode InViewMode, const FText& InText)
		{
			// add mode selection button
			ViewModeSelectorsBox->AddSlot()
			.AutoWidth()
			.Padding(4)
			[
				SNew(SCheckBox)
				.Style(&FAppStyle::GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
				.IsChecked( this, &SIoStoreAnalysisTab::IsViewModeSelected, InViewMode )
				.OnCheckStateChanged( this, &SIoStoreAnalysisTab::OnViewModeCheckStateChange, InViewMode )
				[
					SNew(STextBlock)
						.Text(InText)
				]
			];

			// return a new slot for the mode's panel
			return ViewModeSwitcher->AddSlot();
		};

		// add the view modes
		TSharedRef<FIoStoreActivityTable> ActivityTable = MakeShared<FIoStoreActivityTable>();
		ActivityTable->Reset();
		AddViewMode(EViewMode::ReadActivity, LOCTEXT("ViewMode_ReadActivity", "Read Activity"))
		[
			SAssignNew(ActivityTableTreeView, SActivityTableTreeView, ActivityTable )
		];

		AddViewMode(EViewMode::ReadSizes, LOCTEXT("ViewMode_ReadSizes", "Read Size Histogram"))
		[
			SAssignNew(ReadSizeHistogramView, SIoStoreAnalysisReadSizeHistogramView)
				.ListItemsSource(&ReadSizeHistogramItems)
		];

		// update the data
		RefreshNodes();
	}



	SIoStoreAnalysisTab::~SIoStoreAnalysisTab()
	{
		if (TimingViewSession)
		{
			TimingViewSession->OnTimeMarkerChanged().RemoveAll(this);
			TimingViewSession->OnSelectionChanged().RemoveAll(this);
		}
	}



	void SIoStoreAnalysisTab::SetSession(UE::Insights::Timing::ITimingViewSession* InTimingViewSession, const TraceServices::IAnalysisSession* InAnalysisSession, const FIoStoreInsightsViewSharedState* InSharedStatePtr)
	{
		if (TimingViewSession == InTimingViewSession)
		{
			return;
		}

		if (TimingViewSession)
		{
			TimingViewSession->OnTimeMarkerChanged().RemoveAll(this);
			TimingViewSession->OnSelectionChanged().RemoveAll(this);
			TimingViewSession->OnSelectedEventChanged().RemoveAll(this);
		}

		TimingViewSession = InTimingViewSession;
		AnalysisSession = InAnalysisSession;
		SharedStatePtr = InSharedStatePtr;
		ReadSizeHistogramView->SetAnalysisSession(AnalysisSession);
		ActivityTableTreeView->SetAnalysisSession(AnalysisSession);

		if (InTimingViewSession)
		{
			InTimingViewSession->OnTimeMarkerChanged().AddSP(this, &SIoStoreAnalysisTab::HandleTimeMarkerChanged);
			InTimingViewSession->OnSelectionChanged().AddSP(this, &SIoStoreAnalysisTab::HandleSelectionChanged);
			TimingViewSession->OnSelectedEventChanged().AddSP(this, &SIoStoreAnalysisTab::HandleSelectionEventChanged);
		}

		RefreshNodes();
	}



	bool SIoStoreAnalysisTab::IsSessionSet() const
	{
		return TimingViewSession != nullptr;
	}



	void SIoStoreAnalysisTab::HandleTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker)
	{
		if (!FMath::IsNearlyEqual(StartTime, InTimeMarker) && !FMath::IsNearlyEqual(EndTime, InTimeMarker))
		{
			StartTime = InTimeMarker;
			EndTime = InTimeMarker;

			RefreshNodes();
		}
	}



	void SIoStoreAnalysisTab::HandleSelectionChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime)
	{
		if(InFlags != UE::Insights::Timing::ETimeChangedFlags::Interactive)
		{
			StartTime = InStartTime;
			EndTime = InEndTime;

			RefreshNodes();
		}
	}



	void SIoStoreAnalysisTab::HandleSelectionEventChanged(const TSharedPtr<const ITimingEvent> InEvent)
	{
		if (InEvent)
		{
			const double EventStartTime = InEvent->GetStartTime();
			const double EventEndTime = InEvent->GetEndTime();
			if (!FMath::IsNearlyEqual(StartTime, EventStartTime) && !FMath::IsNearlyEqual(EndTime, EventEndTime))
			{
				StartTime = EventStartTime;
				EndTime = EventEndTime;
				RefreshNodes();
			}
		}
	}



	int32 SIoStoreAnalysisTab::GetSelectedViewModeIndex() const
	{
		return (int32)ViewMode;
	}



	ECheckBoxState SIoStoreAnalysisTab::IsViewModeSelected(EViewMode Mode) const
	{
		return (Mode == ViewMode) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}



	void SIoStoreAnalysisTab::OnViewModeCheckStateChange(ECheckBoxState State, EViewMode Mode)
	{
		if (State == ECheckBoxState::Checked)
		{
			ViewMode = Mode;
		}
	}



	void SIoStoreAnalysisTab::RefreshNodes()
	{
		if (TimingViewSession && AnalysisSession && SharedStatePtr)
		{
			if (EndTime < 0)
			{
				EndTime = +std::numeric_limits<double>::infinity();
			}

			if (StartTime <= EndTime && EndTime >= 0.0)
			{
				const FIoStoreInsightsProvider* Provider = AnalysisSession->ReadProvider<FIoStoreInsightsProvider>(IIoStoreInsightsProvider::ProviderName);
				if (Provider)
				{
					TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
					RefreshNodes_IoStoreActivity(Provider);
				}
			}
		}

		ReadSizeHistogramView->RebuildList();
		ActivityTableTreeView->SetRange(StartTime, EndTime);
	}



	void SIoStoreAnalysisTab::RefreshNodes_IoStoreActivity(const FIoStoreInsightsProvider* Provider)
	{
		const IIoStoreInsightsProvider* IoStoreActivityProvider = AnalysisSession ? AnalysisSession->ReadProvider<IIoStoreInsightsProvider>(FIoStoreInsightsProvider::ProviderName) : nullptr;

		uint32 TotalReads = 0;

		uint64 MinQuantizedReadSize = std::numeric_limits<uint64>::max();
		uint64 MaxQuantizedReadSize = 0;
		TMap<uint64,TSharedPtr<Private::FReadSizeHistogramItem>> QuantizedSizeToHisogramItemMap;
		ReadSizeHistogramItems.Reset();

		Provider->EnumerateIoStoreRequests([this, &TotalReads, &MinQuantizedReadSize, &MaxQuantizedReadSize, &QuantizedSizeToHisogramItemMap](const FIoStoreRequest& IoStoreRequest, const IIoStoreInsightsProvider::Timeline& Timeline)
		{
			Timeline.EnumerateEvents(StartTime, EndTime, [this, &TotalReads, &MinQuantizedReadSize, &MaxQuantizedReadSize, &QuantizedSizeToHisogramItemMap](double EventStartTime, double EventEndTime, uint32 EventDepth, const FIoStoreActivity* IoStoreActivity)
			{
				if (IoStoreActivity->ActivityType == EIoStoreActivityType::Request_Read && IoStoreActivity->EndTime > 0 && !IoStoreActivity->Failed)
				{
					// build the table of quantized reads
					uint64 QuantizedReadSize = FPlatformMath::RoundUpToPowerOfTwo64(IoStoreActivity->ActualSize);
					MaxQuantizedReadSize = FMath::Max(MaxQuantizedReadSize, QuantizedReadSize);
					MinQuantizedReadSize = FMath::Min(MinQuantizedReadSize, QuantizedReadSize);

					double Duration = (IoStoreActivity->EndTime - IoStoreActivity->StartTime);

					TSharedPtr<Private::FReadSizeHistogramItem> HistogramItem;
					if (!QuantizedSizeToHisogramItemMap.Contains(QuantizedReadSize))
					{
						HistogramItem = MakeShared<Private::FReadSizeHistogramItem>();
						HistogramItem->QuantizedReadSize = QuantizedReadSize;
						HistogramItem->MinDuration = Duration;
						HistogramItem->MaxDuration = Duration;
						QuantizedSizeToHisogramItemMap.Add(QuantizedReadSize, HistogramItem);
						ReadSizeHistogramItems.Add(HistogramItem);
					}
					else
					{
						HistogramItem = QuantizedSizeToHisogramItemMap.FindRef(QuantizedReadSize);
					}
					
					TotalReads++;
					HistogramItem->Count++;
					HistogramItem->MinDuration = FMath::Min(Duration, HistogramItem->MinDuration);
					HistogramItem->MaxDuration = FMath::Max(Duration, HistogramItem->MaxDuration);
				}

				return TraceServices::EEventEnumerate::Continue;
			});

			return true;
		});


		// fill in any missing items
		for (uint64 MissingQuantizedReadSize = MinQuantizedReadSize; MissingQuantizedReadSize < MaxQuantizedReadSize || MissingQuantizedReadSize == 0; MissingQuantizedReadSize <<= 1)
		{
			if (!QuantizedSizeToHisogramItemMap.Contains(MissingQuantizedReadSize))
			{
				TSharedPtr<Private::FReadSizeHistogramItem> HistogramItem = MakeShared<Private::FReadSizeHistogramItem>();
				HistogramItem->QuantizedReadSize = MissingQuantizedReadSize;
				QuantizedSizeToHisogramItemMap.Add(MissingQuantizedReadSize, HistogramItem);
				ReadSizeHistogramItems.Add(HistogramItem);
			}
		}

		// sort by read size
		ReadSizeHistogramItems.Sort( []( const TSharedPtr<Private::FReadSizeHistogramItem>& A, const TSharedPtr<Private::FReadSizeHistogramItem>& B)
		{
			return A->QuantizedReadSize < B->QuantizedReadSize;
		});

		// compute percentages
		float MaxPct = 0.0f;
		for (const TSharedPtr<Private::FReadSizeHistogramItem>& HistogramItem : ReadSizeHistogramItems)
		{
			HistogramItem->CountAsPct = (float)HistogramItem->Count / (float)TotalReads;
			MaxPct = FMath::Max(MaxPct, HistogramItem->CountAsPct);
		}
		for (const TSharedPtr<Private::FReadSizeHistogramItem>& HistogramItem : ReadSizeHistogramItems)
		{
			HistogramItem->CountAsPctNormalized = HistogramItem->CountAsPct * (1.0f / MaxPct);
		}
	}

} //namespace UE::IoStoreInsights

#undef LOCTEXT_NAMESPACE
