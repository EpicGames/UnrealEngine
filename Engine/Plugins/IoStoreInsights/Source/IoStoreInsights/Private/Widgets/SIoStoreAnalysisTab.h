// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITimingEvent;
enum class ECheckBoxState : uint8;

namespace UE::Insights::Timing
{
	class ITimingViewSession;
	enum class ETimeChangedFlags;
}

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::IoStoreInsights
{
	class SIoStoreAnalysisReadSizeHistogramView;
	class SActivityTableTreeView;
	class FIoStoreInsightsProvider;
	class FIoStoreInsightsViewSharedState;
	namespace Private { struct FReadSizeHistogramItem; }

	class SIoStoreAnalysisTab : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SIoStoreAnalysisTab) {}
		SLATE_END_ARGS()
		~SIoStoreAnalysisTab();

		void Construct(const FArguments& InArgs);

		void SetSession(UE::Insights::Timing::ITimingViewSession* InTimingViewSession, const TraceServices::IAnalysisSession* InAnalysisSession, const FIoStoreInsightsViewSharedState* InSharedStatePtr);
		bool IsSessionSet() const;

	private:
		enum class EViewMode : uint8
		{
			ReadActivity,
			ReadSizes,
		};

		void HandleTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker);
		void HandleSelectionChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double StartTime, double EndTime);
		void HandleSelectionEventChanged(const TSharedPtr<const ITimingEvent> InEvent);

		int32 GetSelectedViewModeIndex() const;
		ECheckBoxState IsViewModeSelected(EViewMode Mode) const;
		void OnViewModeCheckStateChange(ECheckBoxState State, EViewMode Mode);

		void RefreshNodes();
		void RefreshNodes_IoStoreActivity(const FIoStoreInsightsProvider* Provider);

	private:
		UE::Insights::Timing::ITimingViewSession* TimingViewSession = nullptr;
		const TraceServices::IAnalysisSession* AnalysisSession = nullptr;
		const FIoStoreInsightsViewSharedState* SharedStatePtr = nullptr;

		double StartTime = -1;
		double EndTime = -1;
		EViewMode ViewMode = EViewMode::ReadActivity;

		TArray<TSharedPtr<Private::FReadSizeHistogramItem>> ReadSizeHistogramItems;
		TSharedPtr<SIoStoreAnalysisReadSizeHistogramView> ReadSizeHistogramView;

		TSharedPtr<SActivityTableTreeView> ActivityTableTreeView;
	};

} //namespace UE::IoStoreInsights

