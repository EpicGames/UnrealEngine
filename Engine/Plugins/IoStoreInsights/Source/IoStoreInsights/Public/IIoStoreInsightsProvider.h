// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/AnalysisSession.h"
#include "Model/IntervalTimeline.h"
#include "Templates/Function.h"

namespace UE::IoStoreInsights
{
	enum class EIoStoreActivityType : uint8
	{
		Request_Pending,
		Request_Read,

		Count,
		Invalid = Count
	};

	struct FIoStoreRequest
	{
		uint32 IoStoreRequestIndex = uint32(-1);
		uint32 ChunkIdHash = 0;
		uint8 ChunkType = 0;
		uint64 Offset = 0;
		uint64 Size = 0;
		uint64 CallstackId = 0;
		uint64 PackageId = 0;
		const TCHAR* PackageName = nullptr;
		const TCHAR* ExtraTag = nullptr;
	};

	struct FIoStoreActivity
	{
		const FIoStoreRequest* IoStoreRequest = nullptr;
		double StartTime = 0.0;
		double EndTime = 0.0;
		uint64 ActualSize = 0;
		const TCHAR* BackendName = nullptr;
		uint32 ThreadId = 0;
		EIoStoreActivityType ActivityType = EIoStoreActivityType::Invalid;
		bool Failed = false;
	};

	class IIoStoreInsightsProvider : public TraceServices::IProvider
	{
	public:
		static IOSTOREINSIGHTS_API FName ProviderName;

		typedef TraceServices::ITimeline<FIoStoreActivity*> Timeline;

		virtual ~IIoStoreInsightsProvider() = default;
		virtual void EnumerateIoStoreRequests(TFunctionRef<bool(const FIoStoreRequest&, const Timeline&)> Callback) const = 0;

		virtual const FIoStoreRequest& GetIoStoreRequest(uint32 IoStoreRequestIndex) const = 0;
	};

	IOSTOREINSIGHTS_API const IIoStoreInsightsProvider* ReadIoStoreInsightsProvider(const TraceServices::IAnalysisSession& Session);

	IOSTOREINSIGHTS_API const TCHAR* LexToString(EIoStoreActivityType ActivityType);

} // namespace UE::IoStoreInsights
